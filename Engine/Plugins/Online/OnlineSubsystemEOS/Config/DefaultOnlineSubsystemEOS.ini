[CoreRedirects]
	
+PropertyRedirects=(OldName="ArtifactSettings.EncryptionKey", NewName="ArtifactSettings.ClientEncryptionKey")
