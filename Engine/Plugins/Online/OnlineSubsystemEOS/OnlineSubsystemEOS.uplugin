{
	"FileVersion": 3,

	"FriendlyName": "Online Subsystem EOS",
	"Version": 1,
	"VersionName": "1.0",
	"Description": "Online Subsystem for Epic Online Services",
	"Category": "Online Platform",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://dev.epicgames.com/services",
	"EnabledByDefault": false,
	"IsBetaVersion" : false,

	"Modules": [
		{
			"Name": "OnlineSubsystemEOS",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"Linux",
				"LinuxArm64",
				"Android"
			]
		}
	],
	"Plugins": [
		{
			"Name": "OnlineSubsystem",
			"Enabled": true
		},
		{
			"Name": "OnlineSubsystemUtils",
			"Enabled": true
		},
		{
			"Name": "EOSShared",
			"Enabled": true
		},
		{
			"Name": "EOSVoiceChat",
			"Enabled": true
		},
		{
			"Name": "SocketSubsystemEOS",
			"Enabled": true
		},
		{
			"Name": "EOSOverlayInputProvider",
			"Enabled": true
		},
		{
			"Name": "VoiceChat",
			"Enabled": true
		}
	]
}
