// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class OnlineSubsystemEOS : ModuleRules
{
	public OnlineSubsystemEOS(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDefinitions.Add("ONLINESUBSYSTEMEOS_PACKAGE=1");

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"EOSSDK",
				"EOSShared"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreOnline",
				"CoreUObject",
				"Engine",
				"EOSVoiceChat",
				"Json",
				"NetCore",
				"OnlineBase",
				"OnlineSubsystem",
				"OnlineSubsystemUtils",
				"Sockets",
				"SocketSubsystemEOS",
				"VoiceChat"
			}
		);

		PrivateDefinitions.Add("USE_XBL_XSTS_TOKEN=" + (bUseXblXstsToken ? "1" : "0"));
		PrivateDefinitions.Add("USE_PSN_ID_TOKEN=" + (bUsePsnIdToken ? "1" : "0"));
		PrivateDefinitions.Add("ADD_USER_LOGIN_INFO=" + (bAddUserLoginInfo ? "1" : "0"));
		PrivateDefinitions.Add("EOS_AUTH_TOKEN_SAVEGAME_STORAGE=" + (bAuthTokenSavegameStorage ? "1" : "0"));
	}

	protected virtual bool bUseXblXstsToken { get { return false; } }
	protected virtual bool bUsePsnIdToken { get { return false; } }
	protected virtual bool bAddUserLoginInfo { get { return false; } }
	protected virtual bool bAuthTokenSavegameStorage { get { return false; } }
}
