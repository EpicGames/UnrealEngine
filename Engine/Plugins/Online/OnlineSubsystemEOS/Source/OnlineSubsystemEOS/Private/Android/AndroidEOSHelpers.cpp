// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidEOSHelpers.h"

#if WITH_EOS_SDK

void FAndroidEOSHelpers::PlatformTriggerLoginUI(FOnlineSubsystemEOS* InEOSSubsystem, const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate)
{
	ShowAccountPortalUI(InEOSSubsystem, ControllerIndex, Delegate);
}

#endif
