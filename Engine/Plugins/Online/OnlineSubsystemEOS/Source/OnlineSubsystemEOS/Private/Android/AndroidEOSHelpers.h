// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EOS_SDK

#include "EOSHelpers.h"

class FAndroidEOSHelpers : public FEOSHelpers
{
public:
	virtual void PlatformTriggerLoginUI(FOnlineSubsystemEOS* EOSSubsystem, const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate) override;
};

using FPlatformEOSHelpers = FAndroidEOSHelpers;

#endif