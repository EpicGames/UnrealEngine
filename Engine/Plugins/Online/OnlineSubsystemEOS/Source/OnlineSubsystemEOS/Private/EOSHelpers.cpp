// Copyright Epic Games, Inc. All Rights Reserved.

#include "EOSHelpers.h"

#if WITH_EOS_SDK

#include "IEOSSDKManager.h"
#include "OnlineError.h"
#include "OnlineSubsystemEOS.h"
#include "OnlineSubsystemEOSPrivate.h"
#include "UserManagerEOS.h"

void FEOSHelpers::GetSystemAuthCredentialsOptions(void*& OutCredentialsOptions)
{
}

void FEOSHelpers::PlatformTriggerLoginUI(FOnlineSubsystemEOS* EOSSubsystem, const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate)
{
	check(EOSSubsystem);

	UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("[%hs] This method is not implemented."), __FUNCTION__);

	const FUniqueNetIdPtr PlayerId = EOSSubsystem->UserManager->GetUniquePlayerId(ControllerIndex);

	EOSSubsystem->ExecuteNextTick([PlayerId, ControllerIndex, Delegate]()
	{
		Delegate.ExecuteIfBound(PlayerId, ControllerIndex, FOnlineError(EOnlineErrorResult::NotImplemented));
	});
}

IEOSPlatformHandlePtr FEOSHelpers::CreatePlatform(EOS_Platform_Options& PlatformOptions)
{
	IEOSSDKManager* SDKManager = IEOSSDKManager::Get();
	if (ensure(SDKManager))
	{
		return SDKManager->CreatePlatform(PlatformOptions);
	}
	return nullptr;
}

void FEOSHelpers::AddProductUserIdMapping(IEOSPlatformHandlePtr PlatformHandlePtr, const EOS_ProductUserId& ProductUserId, int32 LocalUserNum)
{
}

void FEOSHelpers::AddExternalAccountMapping(IEOSPlatformHandlePtr PlatformHandlePtr, const EOS_EpicAccountId& EpicAccountId, int32 LocalUserNum)
{
}

void FEOSHelpers::ShowAccountPortalUI(FOnlineSubsystemEOS* InEOSSubsystem, const int ControllerIndex, const FOnLoginUIClosedDelegate& Delegate)
{
	check(InEOSSubsystem != nullptr);
	FDelegateHandle* DelegateHandle = new FDelegateHandle;
	*DelegateHandle = InEOSSubsystem->UserManager->AddOnLoginCompleteDelegate_Handle(ControllerIndex, FOnLoginCompleteDelegate::CreateRaw(this, &FEOSHelpers::OnAccountPortalLoginComplete, InEOSSubsystem, Delegate, DelegateHandle));

	FOnlineAccountCredentials* Credentials = new FOnlineAccountCredentials(TEXT("accountportal"), TEXT(""), TEXT(""));
	InEOSSubsystem->UserManager->Login(ControllerIndex, *Credentials);
}

void FEOSHelpers::OnAccountPortalLoginComplete(int ControllerIndex, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& ErrorString, FOnlineSubsystemEOS* InEOSSubsystem, const FOnLoginUIClosedDelegate LoginUIClosedDelegate, FDelegateHandle* LoginDelegateHandle) const
{
	FOnlineError Error(bWasSuccessful);
	Error.SetFromErrorCode(ErrorString);

	InEOSSubsystem->UserManager->ClearOnLoginCompleteDelegate_Handle(ControllerIndex, *LoginDelegateHandle);
	delete LoginDelegateHandle;

	LoginUIClosedDelegate.ExecuteIfBound(UserId.AsShared(), ControllerIndex, Error);
}

#endif // WITH_EOS_SDK