// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EOS_SDK

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif

#include "eos_auth_types.h"
#include "eos_types.h"
#include "Interfaces/OnlineExternalUIInterface.h"

class FOnlineSubsystemEOS;
using IEOSPlatformHandlePtr = TSharedPtr<class IEOSPlatformHandle, ESPMode::ThreadSafe>;

class FEOSHelpers
{
public:
	virtual ~FEOSHelpers() = default;

	virtual void GetSystemAuthCredentialsOptions(void*& OutCredentialsOptions);
	virtual void PlatformTriggerLoginUI(FOnlineSubsystemEOS* EOSSubsystem, const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate);
	virtual IEOSPlatformHandlePtr CreatePlatform(EOS_Platform_Options& PlatformOptions);
	virtual void AddProductUserIdMapping(IEOSPlatformHandlePtr PlatformHandlePtr, const EOS_ProductUserId& ProductUserId, int32 LocalUserNum);
	virtual void AddExternalAccountMapping(IEOSPlatformHandlePtr PlatformHandlePtr, const EOS_EpicAccountId& EpicAccountId, int32 LocalUserNum);

protected:
	/** Shared LoginUI logic that can be used by platforms that support LoginUI */
	void ShowAccountPortalUI(FOnlineSubsystemEOS* InEOSSubsystem, const int ControllerIndex, const FOnLoginUIClosedDelegate& Delegate);

private:
	/** Completion handler for ShowAccountPortalUI */
	void OnAccountPortalLoginComplete(int ControllerIndex, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& ErrorString, FOnlineSubsystemEOS* InEOSSubsystem, const FOnLoginUIClosedDelegate LoginUIClosedDelegate, FDelegateHandle* LoginDelegateHandle) const;
};

#endif // WITH_EOS_SDK