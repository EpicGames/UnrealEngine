// Copyright Epic Games, Inc. All Rights Reserved.

#include "EOSSettings.h"
#include "OnlineSubsystemEOS.h"
#include "OnlineSubsystemEOSPrivate.h"

#include "Algo/Transform.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EOSSettings)

#if WITH_EDITOR
	#include "Misc/MessageDialog.h"
#endif

#define LOCTEXT_NAMESPACE "EOS"

#define INI_SECTION TEXT("/Script/OnlineSubsystemEOS.EOSSettings")

namespace
{
	inline bool IsAnsi(const FString& Source)
	{
		for (const TCHAR& IterChar : Source)
		{
			if (!FChar::IsPrint(IterChar))
			{
				return false;
			}
		}
		return true;
	}

	inline bool IsHex(const FString& Source)
	{
		for (const TCHAR& IterChar : Source)
		{
			if (!FChar::IsHexDigit(IterChar))
			{
				return false;
			}
		}
		return true;
	}

	inline bool ContainsWhitespace(const FString& Source)
	{
		for (const TCHAR& IterChar : Source)
		{
			if (FChar::IsWhitespace(IterChar))
			{
				return true;
			}
		}
		return false;
	}

	static TOptional<FEOSSettings> GCachedSettings;
	static FDelegateHandle GOnConfigSectionsChangedDelegateHandle;
}

FEOSArtifactSettings FArtifactSettings::ToNative() const
{
	FEOSArtifactSettings Native;

	Native.ArtifactName = ArtifactName;
	Native.ClientId = ClientId;
	Native.ClientSecret = ClientSecret;
	Native.DeploymentId = DeploymentId;
	Native.EncryptionKey = ClientEncryptionKey;
	Native.ProductId = ProductId;
	Native.SandboxId = SandboxId;

	return Native;
}

inline FString StripQuotes(const FString& Source)
{
	if (Source.StartsWith(TEXT("\"")))
	{
		return Source.Mid(1, Source.Len() - 2);
	}
	return Source;
}

FEOSArtifactSettings ParseArtifactSettingsFromConfigString(const FString& RawLine)
{
	FEOSArtifactSettings Result;

	const TCHAR* Delims[4] = { TEXT("("), TEXT(")"), TEXT("="), TEXT(",") };
	TArray<FString> Values;
	RawLine.ParseIntoArray(Values, Delims, 4, false);
	for (int32 ValueIndex = 0; ValueIndex < Values.Num(); ValueIndex++)
	{
		if (Values[ValueIndex].IsEmpty())
		{
			continue;
		}

		// Parse which struct field
		if (Values[ValueIndex] == TEXT("ArtifactName"))
		{
			Result.ArtifactName = StripQuotes(Values[ValueIndex + 1]);
		}
		else if (Values[ValueIndex] == TEXT("ClientId"))
		{
			Result.ClientId = StripQuotes(Values[ValueIndex + 1]);
		}
		else if (Values[ValueIndex] == TEXT("ClientSecret"))
		{
			Result.ClientSecret = StripQuotes(Values[ValueIndex + 1]);
		}
		else if (Values[ValueIndex] == TEXT("ProductId"))
		{
			Result.ProductId = StripQuotes(Values[ValueIndex + 1]);
		}
		else if (Values[ValueIndex] == TEXT("SandboxId"))
		{
			Result.SandboxId = StripQuotes(Values[ValueIndex + 1]);
		}
		else if (Values[ValueIndex] == TEXT("DeploymentId"))
		{
			Result.DeploymentId = StripQuotes(Values[ValueIndex + 1]);
		}
		// EncryptionKey is problematic as a key name as it gets removed by IniKeyDenyList, so lots of EOS config has moved to ClientEncryptionKey instead.
		// That specific issue doesn't affect this specific case as it's part of a config _value_, but supporting both names for consistency and back-compat.
		else if (Values[ValueIndex] == TEXT("EncryptionKey") || Values[ValueIndex] == TEXT("ClientEncryptionKey"))
		{
			Result.EncryptionKey = StripQuotes(Values[ValueIndex + 1]);
		}
		ValueIndex++;
	}

	return Result;
}

FEOSSettings::FEOSSettings()
	: SteamTokenType(TEXT("Session"))
	, RTCBackgroundMode(EOS_ERTCBackgroundMode::EOS_RTCBM_KeepRoomsAlive)
	, TickBudgetInMilliseconds(0)
	, TitleStorageReadChunkLength(0)
	, bEnableOverlay(false)
	, bEnableSocialOverlay(false)
	, bEnableEditorOverlay(false)
	, bPreferPersistentAuth(false)
	, bUseEAS(false)
	, bUseEOSConnect(false)
	, bUseEOSRTC(true)
	, bUseNewEcomFlow(false)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	, bUseNewLoginFlow(false)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{

}

FEOSSettings UEOSSettings::GetSettings()
{
	if (UObjectInitialized())
	{
		return UEOSSettings::AutoGetSettings();
	}

	return UEOSSettings::ManualGetSettings();
}

FEOSSettings UEOSSettings::AutoGetSettings()
{
	return GetDefault<UEOSSettings>()->ToNative();
}

const FEOSSettings& UEOSSettings::ManualGetSettings()
{
	if (!GCachedSettings.IsSet())
	{
		GCachedSettings.Emplace();

		GConfig->GetString(INI_SECTION, TEXT("CacheDir"), GCachedSettings->CacheDir, GEngineIni);
		GConfig->GetString(INI_SECTION, TEXT("DefaultArtifactName"), GCachedSettings->DefaultArtifactName, GEngineIni);
		GConfig->GetString(INI_SECTION, TEXT("SteamTokenType"), GCachedSettings->SteamTokenType, GEngineIni);
		GConfig->GetString(INI_SECTION, TEXT("NintendoTokenType"), GCachedSettings->NintendoTokenType, GEngineIni);
		GConfig->GetString(INI_SECTION, TEXT("PlatformConfigName"), GCachedSettings->PlatformConfigName, GEngineIni);
		GCachedSettings->RTCBackgroundMode = EOS_ERTCBackgroundMode::EOS_RTCBM_KeepRoomsAlive;
		FString RTCBackgroundModeStr;
		GConfig->GetString(INI_SECTION, TEXT("RTCBackgroundMode"), RTCBackgroundModeStr, GEngineIni);
		if (!RTCBackgroundModeStr.IsEmpty())
		{
			LexFromString(GCachedSettings->RTCBackgroundMode, *RTCBackgroundModeStr);
		}
		GConfig->GetInt(INI_SECTION, TEXT("TickBudgetInMilliseconds"), GCachedSettings->TickBudgetInMilliseconds, GEngineIni);
		GConfig->GetInt(INI_SECTION, TEXT("TitleStorageReadChunkLength"), GCachedSettings->TitleStorageReadChunkLength, GEngineIni);
		GConfig->GetBool(INI_SECTION, TEXT("bEnableOverlay"), GCachedSettings->bEnableOverlay, GEngineIni);
		GConfig->GetBool(INI_SECTION, TEXT("bEnableSocialOverlay"), GCachedSettings->bEnableSocialOverlay, GEngineIni);
		GConfig->GetBool(INI_SECTION, TEXT("bEnableEditorOverlay"), GCachedSettings->bEnableEditorOverlay, GEngineIni);
		GConfig->GetBool(INI_SECTION, TEXT("bPreferPersistentAuth"), GCachedSettings->bPreferPersistentAuth, GEngineIni);
		GConfig->GetBool(INI_SECTION, TEXT("bUseEAS"), GCachedSettings->bUseEAS, GEngineIni);
		GConfig->GetBool(INI_SECTION, TEXT("bUseEOSConnect"), GCachedSettings->bUseEOSConnect, GEngineIni);
		GConfig->GetBool(INI_SECTION, TEXT("bUseEOSRTC"), GCachedSettings->bUseEOSRTC, GEngineIni);
		GConfig->GetBool(INI_SECTION, TEXT("bUseNamedPlatformConfig"), GCachedSettings->bUseNamedPlatformConfig, GEngineIni);
		GConfig->GetBool(INI_SECTION, TEXT("bUseNewEcomFlow"), GCachedSettings->bUseNewEcomFlow, GEngineIni);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		GConfig->GetBool(INI_SECTION, TEXT("bUseNewLoginFlow"), GCachedSettings->bUseNewLoginFlow, GEngineIni);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		// Artifacts explicitly skipped
		GConfig->GetArray(INI_SECTION, TEXT("TitleStorageTags"), GCachedSettings->TitleStorageTags, GEngineIni);
		GConfig->GetArray(INI_SECTION, TEXT("AuthScopeFlags"), GCachedSettings->AuthScopeFlags, GEngineIni);
	}

	return *GCachedSettings;
}

FEOSSettings UEOSSettings::ToNative() const
{
	FEOSSettings Native;

	Native.CacheDir = CacheDir;
	Native.DefaultArtifactName = DefaultArtifactName;
	Native.SteamTokenType = SteamTokenType;
	Native.NintendoTokenType = NintendoTokenType;
	Native.PlatformConfigName = PlatformConfigName;
	Native.RTCBackgroundMode = EOS_ERTCBackgroundMode::EOS_RTCBM_KeepRoomsAlive;
	if (!RTCBackgroundMode.IsEmpty())
	{
		LexFromString(Native.RTCBackgroundMode, *RTCBackgroundMode);
	}
	Native.TickBudgetInMilliseconds = TickBudgetInMilliseconds;
	Native.TitleStorageReadChunkLength = TitleStorageReadChunkLength;
	Native.bEnableOverlay = bEnableOverlay;
	Native.bEnableSocialOverlay = bEnableSocialOverlay;
	Native.bEnableEditorOverlay = bEnableEditorOverlay;
	Native.bPreferPersistentAuth = bPreferPersistentAuth;
	Native.bUseEAS = bUseEAS;
	Native.bUseEOSConnect = bUseEOSConnect;
	Native.bUseEOSRTC = bUseEOSRTC;
	Native.bUseNamedPlatformConfig = bUseNamedPlatformConfig;
	Native.bUseNewEcomFlow = bUseNewEcomFlow;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Native.bUseNewLoginFlow = bUseNewLoginFlow;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Algo::Transform(Artifacts, Native.Artifacts, &FArtifactSettings::ToNative);
	Native.TitleStorageTags = TitleStorageTags;
	Native.AuthScopeFlags = AuthScopeFlags;

	return Native;
}

bool UEOSSettings::GetSelectedArtifactSettings(FEOSArtifactSettings& OutSettings)
{
	// Get default artifact name from config.
	FString ArtifactName = GetDefaultArtifactName();
	// Prefer -epicapp over config. This generally comes from EGS.
	FParse::Value(FCommandLine::Get(), TEXT("EpicApp="), ArtifactName);
	// Prefer -EOSArtifactNameOverride over previous.
	FParse::Value(FCommandLine::Get(), TEXT("EOSArtifactNameOverride="), ArtifactName);

	FString SandboxId;
	// Get the -epicsandboxid argument. This generally comes from EGS.
	bool bHasSandboxId = FParse::Value(FCommandLine::Get(), TEXT("EpicSandboxId="), SandboxId);
	// Prefer -EpicSandboxIdOverride over previous.
	bHasSandboxId |= FParse::Value(FCommandLine::Get(), TEXT("EpicSandboxIdOverride="), SandboxId);

	FString DeploymentId;
	// Get the -epicdeploymentid argument. This generally comes from EGS.
	bool bHasDeploymentId = FParse::Value(FCommandLine::Get(), TEXT("EpicDeploymentId="), DeploymentId);
	// Prefer -EpicDeploymentIdOverride over previous.
	bHasDeploymentId |= FParse::Value(FCommandLine::Get(), TEXT("EpicDeploymentIdOverride="), DeploymentId);

	bool bSettingsFound = false;

	// Find the config. We have a hierarchy for what to use, depending on what arguments are provided
	//
	// 1. If SandboxId and DeploymentId are provided on command line, find config entry with matching ArtifactName, SandboxId and DeploymentId
	// 2. If we didn't find a config entry, and SandboxId is provided on command line, find config entry with matching ArtifactName and SandboxId
	// 3. If we didn't find a config entry, find config entry with matching ArtifactName
	// 4. If we didn't find a config entry, find config entry with empty ArtifactName
	//
	// Note for most use cases it is sufficient to ignore 1/2/3 and just provide a single artifact config entry with empty ArtifactName,
	// in which case the client id etc specified in that entry will be used in all cases.
	// SandboxId/DeploymentId provided on command line will take precedence over those specified in the config entry.
	// To support running outside of EGS, ensure you provide values for SandboxId and DeploymentId in the artifact config,
	// and DefaultArtifactName in EOSSettings config, to use when -EpicApp, -EpicSandboxId and/or -EpicDeploymentId are not provided.

	// If SandboxId and DeploymentId are both specified, look for settings with matching ArtifactName, SandboxId, and DeploymentId
	if (bHasSandboxId && bHasDeploymentId)
	{
		bSettingsFound = GetArtifactSettings(ArtifactName, SandboxId, DeploymentId, OutSettings);
		UE_CLOG_ONLINE(!bSettingsFound, Verbose, TEXT("%hs ArtifactName=[%s] SandboxId=[%s] DeploymentId=[%s] no settings found for trio, falling back on pair check."),
			__FUNCTION__, *ArtifactName, *SandboxId, *DeploymentId);
	}

	// Fall back on settings with matching ArtifactName and SandboxId
	if (!bSettingsFound && bHasSandboxId)
	{
		bSettingsFound = GetArtifactSettings(ArtifactName, SandboxId, OutSettings);
		UE_CLOG_ONLINE(!bSettingsFound, Verbose, TEXT("%hs ArtifactName=[%s] SandboxId=[%s] no settings found for pair, falling back on just ArtifactName."),
			__FUNCTION__, *ArtifactName, *SandboxId);
	}

	// Fall back on settings with matching ArtifactName.
	if (!bSettingsFound)
	{
		bSettingsFound = GetArtifactSettings(ArtifactName, OutSettings);
		UE_CLOG_ONLINE(!bSettingsFound, Verbose, TEXT("%hs ArtifactName=[%s] no settings found for ArtifactName, falling back on empty ArtifactName."),
			__FUNCTION__, *ArtifactName);
	}

	// Fall back on settings with an empty ArtifactName.
	if (!bSettingsFound)
	{
		bSettingsFound = GetArtifactSettings(FString(), OutSettings);
		UE_CLOG_ONLINE(!bSettingsFound, Verbose, TEXT("%hs No settings found for empty ArtifactName"), __FUNCTION__);
	}

	UE_CLOG_ONLINE(!bSettingsFound, Error, TEXT("%hs ArtifactName=[%s] SandboxId=[%s] DeploymentId=[%s] no settings found."),
		__FUNCTION__, *ArtifactName, *SandboxId, *DeploymentId);

	// Override the found config with command line values
	if (bSettingsFound)
	{
		OutSettings.ArtifactName = ArtifactName;
		if (bHasSandboxId)
		{
			OutSettings.SandboxId = SandboxId;
		}
		if (bHasDeploymentId)
		{
			OutSettings.DeploymentId = DeploymentId;
		}
	}

	return bSettingsFound;
}

FString UEOSSettings::GetDefaultArtifactName()
{
	if (UObjectInitialized())
	{
		return GetDefault<UEOSSettings>()->DefaultArtifactName;
	}

	FString DefaultArtifactName;
	GConfig->GetString(INI_SECTION, TEXT("DefaultArtifactName"), DefaultArtifactName, GEngineIni);
	return DefaultArtifactName;
}

bool UEOSSettings::GetArtifactSettings(const FString& ArtifactName, FEOSArtifactSettings& OutSettings)
{
	return GetArtifactSettingsImpl(ArtifactName, TOptional<FString>(), TOptional<FString>(), OutSettings);
}

bool UEOSSettings::GetArtifactSettings(const FString& ArtifactName, const FString& SandboxId, FEOSArtifactSettings& OutSettings)
{
	return GetArtifactSettingsImpl(ArtifactName, SandboxId, TOptional<FString>(), OutSettings);
}

bool UEOSSettings::GetArtifactSettings(const FString& ArtifactName, const FString& SandboxId, const FString& DeploymentId, FEOSArtifactSettings& OutSettings)
{
	return GetArtifactSettingsImpl(ArtifactName, SandboxId, DeploymentId, OutSettings);
}

bool UEOSSettings::GetArtifactSettingsImpl(const FString& ArtifactName, const TOptional<FString>& SandboxId, const TOptional<FString>& DeploymentId, FEOSArtifactSettings& OutSettings)
{
	if (UObjectInitialized())
	{
		const UEOSSettings* This = GetDefault<UEOSSettings>();
		const FArtifactSettings* Found = This->Artifacts.FindByPredicate([&ArtifactName, &SandboxId, &DeploymentId](const FArtifactSettings& Element)
		{
			return Element.ArtifactName == ArtifactName
				&& (!SandboxId.IsSet() || Element.SandboxId == SandboxId)
				&& (!DeploymentId.IsSet() || Element.DeploymentId == DeploymentId);
		});
		if (Found)
		{
			OutSettings = Found->ToNative();
			return true;
		}
		return false;
	}
	else
	{
		const TArray<FEOSArtifactSettings>& CachedSettings = GetCachedArtifactSettings();
		const FEOSArtifactSettings* Found = CachedSettings.FindByPredicate([&ArtifactName, &SandboxId, &DeploymentId](const FEOSArtifactSettings& Element)
		{
			return Element.ArtifactName == ArtifactName
				&& (!SandboxId.IsSet() || Element.SandboxId == SandboxId)
				&& (!DeploymentId.IsSet() || Element.DeploymentId == DeploymentId);
		});
		if (Found)
		{
			OutSettings = *Found;
			return true;
		}
		return false;
	}
}

const TArray<FEOSArtifactSettings>& UEOSSettings::GetCachedArtifactSettings()
{
	static TArray<FEOSArtifactSettings> CachedArtifactSettings = []()
	{
		TArray<FString> ConfigArray;
		GConfig->GetArray(INI_SECTION, TEXT("Artifacts"), ConfigArray, GEngineIni);

		TArray<FEOSArtifactSettings> Result;
		Algo::Transform(ConfigArray, Result, &ParseArtifactSettingsFromConfigString);

		return Result;
	}();
	return CachedArtifactSettings;
}

#if WITH_EDITOR
void UEOSSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property == nullptr)
	{
		Super::PostEditChangeProperty(PropertyChangedEvent);
		return;
	}

	// Turning off the overlay in general turns off the social overlay too
	if (PropertyChangedEvent.Property->GetFName() == FName(TEXT("bEnableOverlay")))
	{
		if (!bEnableOverlay)
		{
			bEnableSocialOverlay = false;
			bEnableEditorOverlay = false;
		}
	}

	// Turning on the social overlay requires the base overlay too
	if (PropertyChangedEvent.Property->GetFName() == FName(TEXT("bEnableSocialOverlay")))
	{
		if (bEnableSocialOverlay)
		{
			bEnableOverlay = true;
		}
	}

	if (PropertyChangedEvent.MemberProperty != nullptr &&
		PropertyChangedEvent.MemberProperty->GetFName() == FName(TEXT("Artifacts")) &&
		(PropertyChangedEvent.ChangeType & EPropertyChangeType::ValueSet))
	{
		// Loop through all entries validating them
		for (FArtifactSettings& Artifact : Artifacts)
		{
			if (!Artifact.ClientId.IsEmpty())
			{
				if (!Artifact.ClientId.StartsWith(TEXT("xyz")))
				{
					FMessageDialog::Open(EAppMsgType::Ok,
						LOCTEXT("ClientIdInvalidMsg", "Client ids created after SDK version 1.5 start with xyz. Double check that you did not use your BPT Client Id instead."));
				}
				if (!IsAnsi(Artifact.ClientId) || ContainsWhitespace(Artifact.ClientId))
				{
					FMessageDialog::Open(EAppMsgType::Ok,
						LOCTEXT("ClientIdNotAnsiMsg", "Client ids must contain ANSI printable characters only with no whitespace"));
					Artifact.ClientId.Empty();
				}
			}

			if (!Artifact.ClientSecret.IsEmpty())
			{
				if (!IsAnsi(Artifact.ClientSecret) || ContainsWhitespace(Artifact.ClientSecret))
				{
					FMessageDialog::Open(EAppMsgType::Ok,
						LOCTEXT("ClientSecretNotAnsiMsg", "ClientSecret must contain ANSI printable characters only with no whitespace"));
					Artifact.ClientSecret.Empty();
				}
			}

			if (!Artifact.ClientEncryptionKey.IsEmpty())
			{
				if (!IsHex(Artifact.ClientEncryptionKey) || Artifact.ClientEncryptionKey.Len() != 64)
				{
					FMessageDialog::Open(EAppMsgType::Ok,
						LOCTEXT("EncryptionKeyNotHexMsg", "ClientEncryptionKey must contain 64 hex characters"));
					Artifact.ClientEncryptionKey.Empty();
				}
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif

void UEOSSettings::ModuleInit()
{
	check(!GOnConfigSectionsChangedDelegateHandle.IsValid());
	GOnConfigSectionsChangedDelegateHandle = FCoreDelegates::TSOnConfigSectionsChanged().AddLambda([](const FString& IniFilename, const TSet<FString>& SectionNames)
		{
			if (IniFilename == GEngineIni && SectionNames.Contains(INI_SECTION))
			{
				GCachedSettings.Reset();
			}
		});
}

void UEOSSettings::ModuleShutdown()
{
	FCoreDelegates::TSOnConfigSectionsChanged().Remove(GOnConfigSectionsChangedDelegateHandle);
	GOnConfigSectionsChangedDelegateHandle.Reset();
	GCachedSettings.Reset();
}

#undef LOCTEXT_NAMESPACE

