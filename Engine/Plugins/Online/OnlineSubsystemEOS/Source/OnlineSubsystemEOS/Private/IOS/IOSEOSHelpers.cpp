// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSEOSHelpers.h"

#if WITH_EOS_SDK

#include "OnlineSubsystemEOS.h"
#include "UserManagerEOS.h"

#if (defined(__IPHONE_13_0) && __IPHONE_OS_VERSION_MAX_ALLOWED >= __IPHONE_13_0)
	#define PRESENTATIONCONTEXT_REQUIRED 1
#else
	#define PRESENTATIONCONTEXT_REQUIRED 0
#endif

#if PRESENTATIONCONTEXT_REQUIRED
#import <Foundation/Foundation.h>
#import <AuthenticationServices/AuthenticationServices.h>
#include "IOS/IOSAppDelegate.h"
#endif

#if PRESENTATIONCONTEXT_REQUIRED
@interface PresentationContext : NSObject <ASWebAuthenticationPresentationContextProviding>
{
}
@end


@implementation PresentationContext

- (ASPresentationAnchor)presentationAnchorForWebAuthenticationSession:(ASWebAuthenticationSession *)session
{
	if ([IOSAppDelegate GetDelegate].Window == nullptr)
	{
		NSLog(@"authorizationController: presentationAnchorForAuthorizationController: error window is NULL");
	}
	return [IOSAppDelegate GetDelegate].Window;
}

@end

static PresentationContext* PresentationContextProvider = nil;
#endif

FIOSEOSHelpers::~FIOSEOSHelpers()
{
#if PRESENTATIONCONTEXT_REQUIRED
	if (PresentationContextProvider != nil)
	{
		[PresentationContextProvider release];
		PresentationContextProvider = nil;
	}
#endif
}

void FIOSEOSHelpers::GetSystemAuthCredentialsOptions(void*& OutCredentialsOptions)
{
#if PRESENTATIONCONTEXT_REQUIRED
	if (PresentationContextProvider == nil)
	{
		PresentationContextProvider = [PresentationContext new];
	}
#endif

	CredentialsOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_IOS_AUTH_CREDENTIALSOPTIONS_API_LATEST, 1);
#if PRESENTATIONCONTEXT_REQUIRED
	CredentialsOptions.PresentationContextProviding = (void*)CFBridgingRetain(PresentationContextProvider);		// SDK will release when consumed
#else
	CredentialsOptions.PresentationContextProviding = nullptr;
#endif

	OutCredentialsOptions = (void*)&CredentialsOptions;
}

void FIOSEOSHelpers::PlatformTriggerLoginUI(FOnlineSubsystemEOS* InEOSSubsystem, const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate)
{
	ShowAccountPortalUI(InEOSSubsystem, ControllerIndex, Delegate);
}

#endif

