// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EOS_SDK

#include "eos_auth_types.h"
#include "eos_ios.h"
#include "Interfaces/OnlineExternalUIInterface.h"
#include "EOSHelpers.h"

class FOnlineSubsystemEOS;

class FIOSEOSHelpers : public FEOSHelpers
{
public:
	virtual ~FIOSEOSHelpers();

	virtual void GetSystemAuthCredentialsOptions(void*& OutCredentialsOptions) override;
	virtual void PlatformTriggerLoginUI(FOnlineSubsystemEOS* EOSSubsystem, const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate) override;

private:
	EOS_IOS_Auth_CredentialsOptions CredentialsOptions;
};

using FPlatformEOSHelpers = FIOSEOSHelpers;

#endif

