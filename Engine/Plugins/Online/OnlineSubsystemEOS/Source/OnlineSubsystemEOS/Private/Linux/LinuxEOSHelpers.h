// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EOS_SDK

#include "EOSHelpers.h"

using FPlatformEOSHelpers = FEOSHelpers;

#endif

