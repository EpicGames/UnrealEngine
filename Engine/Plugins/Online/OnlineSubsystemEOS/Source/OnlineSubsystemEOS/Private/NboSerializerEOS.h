// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OnlineSubsystemEOSTypes.h"
#include "NboSerializerOSS.h"

#if WITH_EOS_SDK

/**
 * Serializes data in network byte order form into a buffer
 */
class FNboSerializeToBufferEOS : public FNboSerializeToBufferOSS
{
public:
	/** Default constructor zeros num bytes*/
	FNboSerializeToBufferEOS() :
		FNboSerializeToBufferOSS(512)
	{
	}

	/** Constructor specifying the size to use */
	FNboSerializeToBufferEOS(uint32 Size) :
		FNboSerializeToBufferOSS(Size)
	{
	}

	/**
	 * Adds EOS session info to the buffer
	 */
 	friend inline FNboSerializeToBufferEOS& operator<<(FNboSerializeToBufferEOS& Ar, const FOnlineSessionInfoEOS& SessionInfo)
 	{
		check(SessionInfo.HostAddr.IsValid());
		// Skip SessionType (assigned at creation)
		Ar << *SessionInfo.SessionId;
		((FNboSerializeToBuffer&)Ar) << *SessionInfo.HostAddr;
		return Ar;
 	}

	/**
	 * Adds string Unique Id to the buffer
	 */
	friend inline FNboSerializeToBufferEOS& operator<<(FNboSerializeToBufferEOS& Ar, const FUniqueNetIdString& UniqueId)
	{
		((FNboSerializeToBuffer&)Ar) << UniqueId.UniqueNetIdStr;
		return Ar;
	}
};

/**
 * Class used to write data into packets for sending via system link
 */
class FNboSerializeFromBufferEOS : public FNboSerializeFromBufferOSS
{
public:
	/**
	 * Initializes the buffer, size, and zeros the read offset
	 */
	FNboSerializeFromBufferEOS(uint8* Packet,int32 Length) :
		FNboSerializeFromBufferOSS(Packet,Length)
	{
	}

	/**
	 * Reads EOS session info from the buffer
	 */
 	friend inline FNboSerializeFromBufferEOS& operator>>(FNboSerializeFromBufferEOS& Ar, FOnlineSessionInfoEOS& SessionInfo)
 	{
		check(SessionInfo.HostAddr.IsValid());

		FString SessionId;
		Ar >> SessionId;
		// Skip SessionType (assigned at creation)
		Ar >> *SessionInfo.HostAddr;

		SessionInfo.SessionId = FUniqueNetIdString::Create(MoveTemp(SessionId), FName("EOS"));

		return Ar;
 	}

	/**
	 * Reads string Unique Id from the buffer
	 */
	friend inline FNboSerializeFromBufferEOS& operator>>(FNboSerializeFromBufferEOS& Ar, FUniqueNetIdString& UniqueId)
	{
		Ar >> UniqueId.UniqueNetIdStr;
		return Ar;
	}
};

#endif
