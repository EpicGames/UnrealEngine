// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineAchievementsEOS.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemEOS.h"
#include "OnlineSubsystemEOSPrivate.h"
#include "OnlineSubsystemEOSTypes.h"
#include "OnlineStatsEOS.h"
#include "UserManagerEOS.h"
#include "Misc/ConfigCacheIni.h"

#if WITH_EOS_SDK
#include "eos_achievements.h"

FOnlineAchievementsEOS::FOnlineAchievementsEOS(FOnlineSubsystemEOS* InSubsystem)
	: EOSSubsystem(InSubsystem)
{
}

void FOnlineAchievementsEOS::WriteAchievements(const FUniqueNetId& PlayerId, FOnlineAchievementsWriteRef& WriteObject, const FOnAchievementsWrittenDelegate& Delegate)
{
	UnlockAchievements(PlayerId, WriteObject, Delegate);
}

typedef TEOSCallback<EOS_Achievements_OnUnlockAchievementsCompleteCallback, EOS_Achievements_OnUnlockAchievementsCompleteCallbackInfo, FOnlineAchievementsEOS> FUnlockAchievementsCallback;

void FOnlineAchievementsEOS::UnlockAchievements(const FUniqueNetId& PlayerId, FOnlineAchievementsWriteRef& WriteObject, const FOnAchievementsWrittenDelegate& Delegate)
{
	const int32 LocalUserId = EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(PlayerId);
	if (LocalUserId < 0)
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("Can't unlock achievements for non-local user (%s)"), *PlayerId.ToString());
		Delegate.ExecuteIfBound(PlayerId, false);
		return;
	}

	TArray<FString> InAchievementIds;
	WriteObject->Properties.GenerateKeyArray(InAchievementIds);
	TArray<FTCHARToUTF8> AchievementIdConverters; // We can't use StringCast<UTF8CHAR> because it's non-copyable, and the array will make a copy
	AchievementIdConverters.Reserve(InAchievementIds.Num());
	TArray<const char*> AchievementIdPtrs;
	AchievementIdPtrs.Reserve(InAchievementIds.Num());
	for (const FString& AchievementId : InAchievementIds)
	{
		const FTCHARToUTF8& Converter = AchievementIdConverters.Emplace_GetRef(*AchievementId);
		AchievementIdPtrs.Emplace(Converter.Get());
	}

	EOS_Achievements_UnlockAchievementsOptions Options = {};
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_ACHIEVEMENTS_UNLOCKACHIEVEMENTS_API_LATEST, 1);
	Options.UserId = EOSSubsystem->UserManager->GetLocalProductUserId(LocalUserId); // TODO: The parameter is not called LocalUserId, does that mean that this API works for non-local users?
	Options.AchievementIds = AchievementIdPtrs.GetData();
	Options.AchievementsCount = AchievementIdPtrs.Num();

	FUnlockAchievementsCallback* CallbackObj = new FUnlockAchievementsCallback(FOnlineAchievementsEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, InAchievementIds, LambdaPlayerId = PlayerId.AsShared(), Delegate](const EOS_Achievements_OnUnlockAchievementsCompleteCallbackInfo* Data) mutable
	{
		bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
		if (bWasSuccessful)
		{
			if (TArray<FOnlineAchievement>* Achievements = CachedAchievementsMap.Find(LambdaPlayerId))
			{
				UE_LOG_ONLINE_ACHIEVEMENTS(Verbose, TEXT("(%d) achievements unlocked. Updating achievement cache for user (%s)"), (uint32)Data->AchievementsCount, *LambdaPlayerId->ToString());

				for (const FString& AchievementId : InAchievementIds)
				{
					for (FOnlineAchievement& Achievement : *Achievements)
					{
						if (Achievement.Id == AchievementId)
						{
							Achievement.Progress = 1.0f;
							TriggerOnAchievementUnlockedDelegates(*LambdaPlayerId, AchievementId);
							break;
						}
					}
				}				
			}
			else
			{
				// Achievements were never queried and cached prior to calling UnlockAchievements. Returning success and logging that the achievement cache is empty
				UE_LOG_ONLINE_ACHIEVEMENTS(Verbose, TEXT("Achievements unlocked for user (%s). No cached achievements for this user, please call QueryAchievements to build the achievement cache."), *LambdaPlayerId->ToString());
				
				for (const FString& AchievementId : InAchievementIds)
				{
					TriggerOnAchievementUnlockedDelegates(*LambdaPlayerId, AchievementId);
				}
			}
		}
		else
		{
			UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("EOS_Achievements_UnlockAchievements failed with error code (%s)"), *LexToString(Data->ResultCode));
		}

		Delegate.ExecuteIfBound(*LambdaPlayerId, bWasSuccessful);
	};

	EOS_Achievements_UnlockAchievements(EOSSubsystem->AchievementsHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
}

typedef TEOSCallback<EOS_Achievements_OnQueryPlayerAchievementsCompleteCallback, EOS_Achievements_OnQueryPlayerAchievementsCompleteCallbackInfo, FOnlineAchievementsEOS> FQueryProgressCallback;

void FOnlineAchievementsEOS::QueryAchievements(const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate)
{
	int32 LocalUserId = EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(PlayerId);
	if (LocalUserId < 0)
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("Can't query achievement progress for non-local user (%s)"), *PlayerId.ToString());
		Delegate.ExecuteIfBound(PlayerId, false);
		return;
	}

	EOS_Achievements_QueryPlayerAchievementsOptions Options = { };
	Options.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_ACHIEVEMENTS_QUERYPLAYERACHIEVEMENTS_API_LATEST, 2);
	Options.LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(LocalUserId);
	Options.TargetUserId = Options.LocalUserId;

	FQueryProgressCallback* CallbackObj = new FQueryProgressCallback(FOnlineAchievementsEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, LambdaPlayerId = PlayerId.AsShared(), OnComplete = FOnQueryAchievementsCompleteDelegate(Delegate)](const EOS_Achievements_OnQueryPlayerAchievementsCompleteCallbackInfo* Data)
	{
		bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
		if (bWasSuccessful)
		{
			TArray<FOnlineAchievement>& PlayerAchievements = CachedAchievementsMap.Add(LambdaPlayerId);

			int32 LocalUserNum = EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(*LambdaPlayerId);
			EOS_ProductUserId UserId = EOSSubsystem->UserManager->GetLocalProductUserId(LocalUserNum);

			EOS_Achievements_GetPlayerAchievementCountOptions CountOptions = { };
			CountOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_ACHIEVEMENTS_GETPLAYERACHIEVEMENTCOUNT_API_LATEST, 1);
			CountOptions.UserId = UserId;
			uint32 Count = EOS_Achievements_GetPlayerAchievementCount(EOSSubsystem->AchievementsHandle, &CountOptions);

			EOS_Achievements_CopyPlayerAchievementByIndexOptions CopyOptions = { };
			CopyOptions.ApiVersion = 2;
			UE_EOS_CHECK_API_MISMATCH(EOS_ACHIEVEMENTS_COPYPLAYERACHIEVEMENTBYINDEX_API_LATEST, 2);
			CopyOptions.LocalUserId = UserId;
			CopyOptions.TargetUserId = UserId;

			for (uint32 Index = 0; Index < Count; Index++)
			{
				CopyOptions.AchievementIndex = Index;

				EOS_Achievements_PlayerAchievement* AchievementEOS = nullptr;
				EOS_EResult Result = EOS_Achievements_CopyPlayerAchievementByIndex(EOSSubsystem->AchievementsHandle, &CopyOptions, &AchievementEOS);
				if (Result == EOS_EResult::EOS_Success)
				{
					FOnlineAchievement& Achievement = PlayerAchievements.Emplace_GetRef();
					Achievement.Id = AchievementEOS->AchievementId;
					Achievement.Progress = AchievementEOS->Progress;

					EOS_Achievements_PlayerAchievement_Release(AchievementEOS);

					if (UE_BUILD_DEBUG)
					{
						UE_LOG_ONLINE_ACHIEVEMENTS(Log, TEXT("Achievement progress (%s)"), *Achievement.ToDebugString());
					}
				}
				else
				{
					UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("EOS_Achievements_CopyPlayerAchievementByIndex() failed with error code (%s)"), *LexToString(Result));
				}
			}
		}
		else
		{
			UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("EOS_Achievements_QueryPlayerAchievements() failed with error code (%s)"), *LexToString(Data->ResultCode));
		}
		OnComplete.ExecuteIfBound(*LambdaPlayerId, bWasSuccessful);
	};
	EOS_Achievements_QueryPlayerAchievements(EOSSubsystem->AchievementsHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
}

typedef TEOSCallback<EOS_Achievements_OnQueryDefinitionsCompleteCallback, EOS_Achievements_OnQueryDefinitionsCompleteCallbackInfo, FOnlineAchievementsEOS> FQueryDefinitionsCallback;

void FOnlineAchievementsEOS::QueryAchievementDescriptions(const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate)
{
	if (CachedAchievementDefinitions.Num())
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Using cached achievement definitions"));
		Delegate.ExecuteIfBound(PlayerId, true);
		return;
	}

	int32 LocalUserId = EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(PlayerId);
	if (LocalUserId < 0)
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("Can't query achievement definitions for non-local user (%s)"), *PlayerId.ToString());
		Delegate.ExecuteIfBound(PlayerId, false);
		return;
	}

	EOS_Achievements_QueryDefinitionsOptions Options = { };
	Options.ApiVersion = 3;
	UE_EOS_CHECK_API_MISMATCH(EOS_ACHIEVEMENTS_QUERYDEFINITIONS_API_LATEST, 3);
	Options.LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(LocalUserId);

	FQueryDefinitionsCallback* CallbackObj = new FQueryDefinitionsCallback(FOnlineAchievementsEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, LambdaPlayerId = PlayerId.AsShared(), OnComplete = FOnQueryAchievementsCompleteDelegate(Delegate)](const EOS_Achievements_OnQueryDefinitionsCompleteCallbackInfo* Data)
	{
		bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
		if (bWasSuccessful)
		{
			EOS_Achievements_GetAchievementDefinitionCountOptions CountOptions = { };
			CountOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_ACHIEVEMENTS_GETACHIEVEMENTDEFINITIONCOUNT_API_LATEST, 1);
			uint32 Count = EOS_Achievements_GetAchievementDefinitionCount(EOSSubsystem->AchievementsHandle, &CountOptions);

			EOS_Achievements_CopyAchievementDefinitionByIndexOptions CopyOptions = { };
			CopyOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_ACHIEVEMENTS_COPYDEFINITIONBYINDEX_API_LATEST, 1);
			CachedAchievementDefinitions.Empty(Count);
			CachedAchievementDefinitionsMap.Empty();

			for (uint32 Index = 0; Index < Count; Index++)
			{
				CopyOptions.AchievementIndex = Index;
				EOS_Achievements_Definition* Definition = nullptr;

				EOS_EResult Result = EOS_Achievements_CopyAchievementDefinitionByIndex(EOSSubsystem->AchievementsHandle, &CopyOptions, &Definition);
				if (Result == EOS_EResult::EOS_Success)
				{
					FOnlineAchievementDesc* Desc = new(CachedAchievementDefinitions) FOnlineAchievementDesc();
					// Work around for the ID not being part of the description
					CachedAchievementDefinitionsMap.Add(Definition->AchievementId, Desc);

					Desc->Title = FText::FromString(UTF8_TO_TCHAR(Definition->DisplayName));
					Desc->LockedDesc = FText::FromString(UTF8_TO_TCHAR(Definition->LockedDescription));
					Desc->UnlockedDesc = FText::FromString(UTF8_TO_TCHAR(Definition->CompletionDescription));
					Desc->bIsHidden = Definition->bIsHidden == EOS_TRUE;

					EOS_Achievements_Definition_Release(Definition);

					if (UE_BUILD_DEBUG)
					{
						UE_LOG_ONLINE_ACHIEVEMENTS(Log, TEXT("Achievement desc is (%s)"), *Desc->ToDebugString());
					}
				}
				else
				{
					UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("EOS_Achievements_CopyAchievementDefinitionByIndex() failed with error code (%s)"), *LexToString(Result));
				}
			}
		}
		else
		{
			UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("EOS_Achievements_QueryDefinitions() failed with error code (%s)"), *LexToString(Data->ResultCode));
		}
		OnComplete.ExecuteIfBound(*LambdaPlayerId, bWasSuccessful);
	};
	EOS_Achievements_QueryDefinitions(EOSSubsystem->AchievementsHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
}

EOnlineCachedResult::Type FOnlineAchievementsEOS::GetCachedAchievement(const FUniqueNetId& PlayerId, const FString& AchievementId, FOnlineAchievement& OutAchievement)
{
	if (const TArray<FOnlineAchievement>* Achievements = CachedAchievementsMap.Find(PlayerId.AsShared()))
	{
		for (const FOnlineAchievement& Achievement : *Achievements)
		{
			if (Achievement.Id == AchievementId)
			{
				OutAchievement = Achievement;
				return EOnlineCachedResult::Success;
			}
		}
	}
	return EOnlineCachedResult::NotFound;
}

EOnlineCachedResult::Type FOnlineAchievementsEOS::GetCachedAchievements(const FUniqueNetId& PlayerId, TArray<FOnlineAchievement>& OutAchievements)
{
	if (const TArray<FOnlineAchievement>* CachedAchievements = CachedAchievementsMap.Find(PlayerId.AsShared()))
	{
		OutAchievements = *CachedAchievements;
		return EOnlineCachedResult::Success;
	}
	return EOnlineCachedResult::NotFound;
}

EOnlineCachedResult::Type FOnlineAchievementsEOS::GetCachedAchievementDescription(const FString& AchievementId, FOnlineAchievementDesc& OutAchievementDesc)
{
	if (CachedAchievementDefinitionsMap.Contains(AchievementId))
	{
		OutAchievementDesc = *CachedAchievementDefinitionsMap[AchievementId];
		return EOnlineCachedResult::Success;
	}
	return EOnlineCachedResult::NotFound;
}

#if !UE_BUILD_SHIPPING
bool FOnlineAchievementsEOS::ResetAchievements(const FUniqueNetId&)
{
	UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("ResetAchievements() is not supported"));
	return false;
}
#endif

#endif
