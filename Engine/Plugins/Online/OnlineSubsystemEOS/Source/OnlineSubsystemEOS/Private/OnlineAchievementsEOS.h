// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Online/CoreOnline.h"
#include "Interfaces/OnlineAchievementsInterface.h"
#include "OnlineSubsystemEOSTypes.h"

class FOnlineSubsystemEOS;

#if WITH_EOS_SDK
#include "eos_achievements_types.h"

/**
 * Interface for interacting with EOS achievements
 */
class FOnlineAchievementsEOS
	: public IOnlineAchievements
	, public TSharedFromThis<FOnlineAchievementsEOS, ESPMode::ThreadSafe>
{
public:
	FOnlineAchievementsEOS() = delete;
	virtual ~FOnlineAchievementsEOS() = default;

	FOnlineAchievementsEOS(FOnlineSubsystemEOS* InSubsystem);

// IOnlineAchievements Interface
	virtual void WriteAchievements(const FUniqueNetId& PlayerId, FOnlineAchievementsWriteRef& WriteObject, const FOnAchievementsWrittenDelegate& Delegate = FOnAchievementsWrittenDelegate())  override;
	virtual void QueryAchievements(const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate = FOnQueryAchievementsCompleteDelegate())  override;
	virtual void QueryAchievementDescriptions(const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate = FOnQueryAchievementsCompleteDelegate())  override;
	virtual EOnlineCachedResult::Type GetCachedAchievement(const FUniqueNetId& PlayerId, const FString& AchievementId, FOnlineAchievement& OutAchievement)  override;
	virtual EOnlineCachedResult::Type GetCachedAchievements(const FUniqueNetId& PlayerId, TArray<FOnlineAchievement>& OutAchievements)  override;
	virtual EOnlineCachedResult::Type GetCachedAchievementDescription(const FString& AchievementId, FOnlineAchievementDesc& OutAchievementDesc)  override;
#if !UE_BUILD_SHIPPING
	virtual bool ResetAchievements(const FUniqueNetId& PlayerId) override;
#endif
// ~IOnlineAchievements Interface

private:
	void UnlockAchievements(const FUniqueNetId& PlayerId, FOnlineAchievementsWriteRef& WriteObject, const FOnAchievementsWrittenDelegate& Delegate);

private:
	/** Reference to the main EOS subsystem */
	FOnlineSubsystemEOS* EOSSubsystem;
	/** Holds the cached info from the last time this was called */
	TUniqueNetIdMap<TArray<FOnlineAchievement>> CachedAchievementsMap;
	/** Holds the cached info from the last time this was called */
	TArray<FOnlineAchievementDesc> CachedAchievementDefinitions;
	/** Hack to work around the fact the description doesn't include the ID */
	TMap<FString, FOnlineAchievementDesc*> CachedAchievementDefinitionsMap;
};

typedef TSharedPtr<FOnlineAchievementsEOS, ESPMode::ThreadSafe> FOnlineAchievementsEOSPtr;
typedef TWeakPtr<FOnlineAchievementsEOS, ESPMode::ThreadSafe> FOnlineAchievementsEOSWeakPtr;

#endif
