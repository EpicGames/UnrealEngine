// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineEntitlementsEOS.h"
#include "EOSSettings.h"
#include "IEOSSDKManager.h"

#if WITH_EOS_SDK

#include "OnlineSubsystemEOS.h"
#include "OnlineSubsystemEOSPrivate.h"
#include "UserManagerEOS.h"
#include "eos_ecom.h"

#define ONLINE_ERROR_NAMESPACE "com.epicgames.oss.eos.error"


FOnlineEntitlementsEOS::FOnlineEntitlementsEOS(FOnlineSubsystemEOS* InSubsystem)
	: EOSSubsystem(InSubsystem)
{
	check(EOSSubsystem != nullptr);
}

typedef TEOSCallback<EOS_Ecom_OnQueryEntitlementsCallback, EOS_Ecom_QueryEntitlementsCallbackInfo, FOnlineEntitlementsEOS> FQueryEntitlementsCallback;

bool FOnlineEntitlementsEOS::QueryEntitlements(const FUniqueNetId& UserId, const FString& Namespace, const FPagedQuery& Page)
{
	const FUniqueNetIdEOS& UserEOSId = FUniqueNetIdEOS::Cast(UserId);
	const EOS_EpicAccountId AccountId = UserEOSId.GetEpicAccountId();
	if (AccountId == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("[%hs] EpicAccountId is null"), __FUNCTION__);
		return false;
	}

	EOS_Ecom_QueryEntitlementsOptions Options = { };
	Options.ApiVersion = 3;
	UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_QUERYENTITLEMENTS_API_LATEST, 3);
	Options.LocalUserId = AccountId;
	Options.bIncludeRedeemed = EOS_TRUE;
	const auto NamespaceUtf8 = StringCast<UTF8CHAR>(*Namespace); 
	Options.OverrideCatalogNamespace = Namespace.IsEmpty() ? nullptr : (const char*) NamespaceUtf8.Get();

	FQueryEntitlementsCallback* CallbackObj = new FQueryEntitlementsCallback(FOnlineEntitlementsEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, LambdaUserId = UserId.AsShared(), LambdaNamespace = FString(Namespace)](const EOS_Ecom_QueryEntitlementsCallbackInfo* Data)
	{
		if (Data->ResultCode == EOS_EResult::EOS_Success)
		{
			EOS_Ecom_GetEntitlementsCountOptions CountOptions = { };
			CountOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_GETENTITLEMENTSCOUNT_API_LATEST, 1);
			CountOptions.LocalUserId = Data->LocalUserId;
			uint32 Count = EOS_Ecom_GetEntitlementsCount(EOSSubsystem->EcomHandle, &CountOptions);

			TArray<TSharedRef<FOnlineEntitlement>> NewEntitlements;
			NewEntitlements.Reserve(Count);

			EOS_Ecom_CopyEntitlementByIndexOptions CopyEntitlementOptions = { };
			CopyEntitlementOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_COPYENTITLEMENTBYINDEX_API_LATEST, 1);
			CopyEntitlementOptions.LocalUserId = Data->LocalUserId;

			for (uint32 Index = 0; Index < Count; Index++)
			{
				CopyEntitlementOptions.EntitlementIndex = Index;

				EOS_Ecom_Entitlement* Entitlement = nullptr;
				EOS_EResult CopyEntitlementResult = EOS_Ecom_CopyEntitlementByIndex(EOSSubsystem->EcomHandle, &CopyEntitlementOptions, &Entitlement);
				if (CopyEntitlementResult == EOS_EResult::EOS_Success || CopyEntitlementResult == EOS_EResult::EOS_Ecom_EntitlementStale)
				{
					TSharedRef<FOnlineEntitlement> NewEntitlement = MakeShared<FOnlineEntitlement>();
					NewEntitlement->Id = Entitlement->EntitlementId ? StringCast<TCHAR>(Entitlement->EntitlementId).Get() : FString();
					NewEntitlement->ItemId = Entitlement->CatalogItemId ? StringCast<TCHAR>(Entitlement->CatalogItemId).Get() : FString();
					NewEntitlement->Name = Entitlement->EntitlementName ? StringCast<TCHAR>(Entitlement->EntitlementName).Get() : FString();

					NewEntitlement->EndDate = Entitlement->EndTimestamp > 0 ? FDateTime::FromUnixTimestamp(Entitlement->EndTimestamp).ToFormattedString(TEXT("%Y-%m-%d_%H-%M-%S")) : TEXT("");
					NewEntitlement->Namespace = LambdaNamespace;
					// There is no consumed count in the EOS SDK - This field is used to identify entitlements that have been redeemed
					NewEntitlement->ConsumedCount = Entitlement->bRedeemed ? 1 : 0;

					EOS_Ecom_CatalogItem* CatalogItem = nullptr;
					EOS_Ecom_CopyItemByIdOptions CopyCatalogItemOptions = {};
					CopyCatalogItemOptions.ApiVersion = 1;
					UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_COPYITEMBYID_API_LATEST, 1);
					CopyCatalogItemOptions.ItemId = Entitlement->CatalogItemId;
					CopyCatalogItemOptions.LocalUserId = Data->LocalUserId;
					EOS_EResult CopyCatalogItemResult = EOS_Ecom_CopyItemById(EOSSubsystem->EcomHandle, &CopyCatalogItemOptions, &CatalogItem);

					if (CopyCatalogItemResult == EOS_EResult::EOS_Success || CopyCatalogItemResult == EOS_EResult::EOS_Ecom_CatalogItemStale)
					{
						// EOS_Ecom_CopyItemById will return EOS_NotFound is QueryOffers has NOT been called OR if the catalogItemId doesn't tie back to a specific offer (bundles, season packs, etc...)
						NewEntitlement->bIsConsumable = CatalogItem->ItemType == EOS_EEcomItemType::EOS_EIT_Consumable;
						EOS_Ecom_CatalogItem_Release(CatalogItem);
					}
					else
					{
						UE_LOG_ONLINE(Verbose, TEXT("[FOnlineEntitlementsEOS::QueryEntitlements] EOS_Ecom_CopyItemById: failed with error (%s)"), *LexToString(CopyCatalogItemResult));
						// Default bIsConsumbale to true
						NewEntitlement->bIsConsumable = true;
					}
					NewEntitlements.Add(NewEntitlement);
				}
				else
				{
					UE_LOG_ONLINE(Error, TEXT("[FOnlineEntitlementsEOS::QueryEntitlements] EOS_Ecom_CopyEntitlementByIndex: failed with error (%s)"), *LexToString(CopyEntitlementResult));
					continue;
				}

				EOS_Ecom_Entitlement_Release(Entitlement);
			}

			CachedEntitlementsMap.FindOrAdd(LambdaUserId) = MoveTemp(NewEntitlements);
			TriggerOnQueryEntitlementsCompleteDelegates(true, *LambdaUserId, LambdaNamespace, "");
		}
		else
		{
			FString ErrorString = FString::Printf(TEXT("result code (%s)"), *LexToString(Data->ResultCode));
			UE_LOG_ONLINE(Error, TEXT("[FOnlineEntitlementsEOS::QueryEntitlements] %s"), *ErrorString);
			TriggerOnQueryEntitlementsCompleteDelegates(false,*LambdaUserId, LambdaNamespace, ErrorString);
		}
	};
	
	EOS_Ecom_QueryEntitlements(EOSSubsystem->EcomHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
	return true;
}

TSharedPtr<FOnlineEntitlement> FOnlineEntitlementsEOS::GetEntitlement(const FUniqueNetId& UserId, const FUniqueEntitlementId& EntitlementId)
{
	const FUniqueNetIdEOS& UserEOSId = FUniqueNetIdEOS::Cast(UserId);
	const EOS_EpicAccountId AccountId = UserEOSId.GetEpicAccountId();
	if (AccountId == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("[%hs] EpicAccountId is null"), __FUNCTION__);
		return nullptr;
	}

	const TArray<TSharedRef<FOnlineEntitlement>>* CachedUserEntitlements = CachedEntitlementsMap.Find(UserId.AsShared());
	if (!CachedUserEntitlements || CachedUserEntitlements->IsEmpty())
	{
		UE_LOG_ONLINE(Warning, TEXT("[%hs] CachedEntitlements empty for user %s, call QueryEntitlements"), __FUNCTION__, *UserId.ToString());
		return nullptr;
	}

	TSharedPtr<FOnlineEntitlement> Entitlement = nullptr;
	for (const TSharedRef<FOnlineEntitlement>& CachedUserEntitlement : *CachedUserEntitlements)
	{
		if (CachedUserEntitlement->Id == EntitlementId)
		{
			Entitlement = CachedUserEntitlement;
			break;
		}
	}

	return Entitlement;
}

TSharedPtr<FOnlineEntitlement> FOnlineEntitlementsEOS::GetItemEntitlement(const FUniqueNetId& UserId, const FString& ItemId)
{
	const FUniqueNetIdEOS& UserEOSId = FUniqueNetIdEOS::Cast(UserId);
	const EOS_EpicAccountId AccountId = UserEOSId.GetEpicAccountId();
	if (AccountId == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("[%hs] EpicAccountId is null"), __FUNCTION__);
		return nullptr;
	}

	const TArray<TSharedRef<FOnlineEntitlement>>* CachedUserEntitlements = CachedEntitlementsMap.Find(UserId.AsShared());
	if (!CachedUserEntitlements || CachedUserEntitlements->IsEmpty())
	{
		UE_LOG_ONLINE(Warning, TEXT("[%hs] CachedEntitlements empty for user %s, call QueryEntitlements"), __FUNCTION__, *UserId.ToString());
		return nullptr;
	}

	TSharedPtr<FOnlineEntitlement> Entitlement = nullptr;
	for (const TSharedRef<FOnlineEntitlement>& CachedUserEntitlement : *CachedUserEntitlements)
	{
		if (CachedUserEntitlement->ItemId == ItemId)
		{
			Entitlement = CachedUserEntitlement;
			break;
		}
	}

	return Entitlement;
}

void FOnlineEntitlementsEOS::GetAllEntitlements(const FUniqueNetId& UserId, const FString& Namespace, TArray<TSharedRef<FOnlineEntitlement>>& OutUserEntitlements)
{
	OutUserEntitlements.Empty();

	const FUniqueNetIdEOS& UserEOSId = FUniqueNetIdEOS::Cast(UserId);
	const EOS_EpicAccountId AccountId = UserEOSId.GetEpicAccountId();
	if (AccountId == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("[%hs] EpicAccountId is null"), __FUNCTION__);
		return;
	}

	const TArray<TSharedRef<FOnlineEntitlement>>* CachedUserEntitlements = CachedEntitlementsMap.Find(UserId.AsShared());
	if (!CachedUserEntitlements || CachedUserEntitlements->IsEmpty())
	{
		UE_LOG_ONLINE(Warning, TEXT("[%hs] CachedEntitlements empty for user %s, call QueryEntitlements"), __FUNCTION__, *UserId.ToString());
		return;
	}

	if (Namespace.IsEmpty())
	{
		OutUserEntitlements = *CachedUserEntitlements;
	}
	else
	{
		for (const TSharedRef<FOnlineEntitlement>& CachedUserEntitlement: *CachedUserEntitlements)
		{
			if (CachedUserEntitlement->Namespace == Namespace)
			{
				OutUserEntitlements.Add(CachedUserEntitlement);
			}
		}
	}
}

bool FOnlineEntitlementsEOS::HandleEntitlementsExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (!EOSSubsystem || !EOSSubsystem->UserManager.IsValid() || !EOSSubsystem->EntitlementsInterfacePtr.IsValid())
	{
		UE_LOG_ONLINE(Error, TEXT("[%hs] EOSSubsystem or UserManager or EntitlementsInterface invalid"), __FUNCTION__);
		return false;
	}

	if (FParse::Command(&Cmd, TEXT("ENTITLEMENTS")))
	{
		TSharedPtr<FDelegateHandle> DelegateHandle = MakeShared<FDelegateHandle>();
			
		*DelegateHandle = AddOnQueryEntitlementsCompleteDelegate_Handle(FOnQueryEntitlementsComplete::FDelegate::CreateLambda([this, DelegateHandle](bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Namespace, const FString& Error)
		{
			if (DelegateHandle.IsValid())
			{
				ClearOnQueryEntitlementsCompleteDelegate_Handle(*DelegateHandle);
				DelegateHandle->Reset();
			}

			if (bWasSuccessful)
			{
				const TArray<TSharedRef<FOnlineEntitlement>>* CachedUserEntitlements = CachedEntitlementsMap.Find(UserId.AsShared());
				if (!CachedUserEntitlements || CachedUserEntitlements->IsEmpty())
				{
					UE_LOG_ONLINE(Log, TEXT("[FOnlineEntitlementsEOS::HandleEntitlementsExec] CachedEntitlements empty for user %s, call QueryEntitlements"), *UserId.ToString());
					return;
				}

				for (const TSharedRef<FOnlineEntitlement>& CachedUserEntitlement : *CachedUserEntitlements)
				{
					UE_LOG_ONLINE(Log, TEXT("[FOnlineEntitlementsEOS::HandleEntitlementsExec] EntitlementId: %s"), *CachedUserEntitlement->Id);
					UE_LOG_ONLINE(Log, TEXT("[FOnlineEntitlementsEOS::HandleEntitlementsExec] ItemId: %s"), *CachedUserEntitlement->ItemId);
					UE_LOG_ONLINE(Log, TEXT("[FOnlineEntitlementsEOS::HandleEntitlementsExec] Name: %s"), *CachedUserEntitlement->Name);
					UE_LOG_ONLINE(Log, TEXT("[FOnlineEntitlementsEOS::HandleEntitlementsExec] bIsConsumable: %s"), *LexToString(CachedUserEntitlement->bIsConsumable));
				}
			}
			else
			{
				UE_LOG_ONLINE(Error, TEXT("[FOnlineEntitlementsEOS::HandleEntitlementsExec] error (%s)"), *Error);
			}
		}));

		QueryEntitlements(*EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(EOSSubsystem->UserManager->GetDefaultLocalUser()), "");
		return true;
	}
	return false;
}

#endif

