// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Online/CoreOnline.h"
#include "Interfaces/OnlineEntitlementsInterface.h"
#include "OnlineSubsystemEOSTypes.h"

#if WITH_EOS_SDK
#include "eos_ecom_types.h"

class UWorld;

/**
 * Query entitlements
 */
class FOnlineEntitlementsEOS :
	public IOnlineEntitlements,
	public TSharedFromThis<FOnlineEntitlementsEOS, ESPMode::ThreadSafe>
{
public:
	virtual ~FOnlineEntitlementsEOS() = default;
	
	virtual bool QueryEntitlements(const FUniqueNetId& UserId, const FString& Namespace, const FPagedQuery& Page = FPagedQuery()) override;
	virtual TSharedPtr<FOnlineEntitlement> GetEntitlement(const FUniqueNetId& UserId, const FUniqueEntitlementId& EntitlementId) override;
	virtual TSharedPtr<FOnlineEntitlement> GetItemEntitlement(const FUniqueNetId& UserId, const FString& ItemId) override;
	virtual void GetAllEntitlements(const FUniqueNetId& UserId, const FString& Namespace, TArray<TSharedRef<FOnlineEntitlement>>& OutUserEntitlements) override;
	
	FOnlineEntitlementsEOS(FOnlineSubsystemEOS* InSubsystem);
	
	bool HandleEntitlementsExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);

private:
	/** Default constructor disabled */
	FOnlineEntitlementsEOS() = delete;

	/** List of entitlements for the player */
	TMap<FUniqueNetIdRef, TArray<TSharedRef<FOnlineEntitlement>>> CachedEntitlementsMap;

	/** Reference to the main EOS subsystem */
	FOnlineSubsystemEOS* EOSSubsystem;
};

typedef TSharedPtr<FOnlineEntitlementsEOS, ESPMode::ThreadSafe> FOnlineEntitlementsEOSPtr;
typedef TWeakPtr<FOnlineEntitlementsEOS, ESPMode::ThreadSafe> FOnlineEntitlementsEOSWeakPtr;

#endif
