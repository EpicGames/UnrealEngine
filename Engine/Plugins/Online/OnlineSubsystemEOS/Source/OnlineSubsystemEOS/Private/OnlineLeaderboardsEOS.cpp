// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineLeaderboardsEOS.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemEOS.h"
#include "OnlineSubsystemEOSPrivate.h"
#include "OnlineSubsystemEOSTypes.h"
#include "UserManagerEOS.h"
#include "OnlineStatsEOS.h"

#if WITH_EOS_SDK
#include "eos_leaderboards.h"
#include "eos_userinfo.h"

namespace Private
{
struct FRankedUserInfo
{
	FUniqueNetIdEOSRef NetId;
	EOS_ProductUserId ProductUserId;
	FString Nickname;
	uint32 Rank;
	int32 Score;

	FRankedUserInfo(const FUniqueNetIdEOSRef InNetId, EOS_ProductUserId InProductUserId, FString InNickname, uint32 InRank, int32 InScore)
		: NetId(InNetId)
		, ProductUserId(InProductUserId)
		, Nickname(MoveTemp(InNickname))
		, Rank(InRank)
		, Score(InScore)
	{
	}
};

/** Collect rank records into FRankedUserInfo array, resolving against known NetIds */
void CollectRankedUsers(EOS_HLeaderboards LeaderboardsHandle, uint32 StartIndex, uint32 EndIndex,
	const TMap<EOS_ProductUserId, FUniqueNetIdEOSRef>& ResolvedUniqueNetIds,
	TArray<FRankedUserInfo>& OutRankedUsers, TArray<EOS_ProductUserId>& OutProductUserIds)
{
	EOS_Leaderboards_CopyLeaderboardRecordByIndexOptions CopyOptions = { };
	CopyOptions.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_LEADERBOARDS_COPYLEADERBOARDRECORDBYINDEX_API_LATEST, 2);

	for (uint32 Index = StartIndex; Index <= EndIndex; Index++)
	{
		CopyOptions.LeaderboardRecordIndex = Index;

		EOS_Leaderboards_LeaderboardRecord* Record = nullptr;
		EOS_EResult Result = EOS_Leaderboards_CopyLeaderboardRecordByIndex(LeaderboardsHandle, &CopyOptions, &Record);
		if (Result == EOS_EResult::EOS_Success)
		{
			if (const FUniqueNetIdEOSRef* NetId = ResolvedUniqueNetIds.Find(Record->UserId))
			{
				FString Nickname = UTF8_TO_TCHAR(Record->UserDisplayName);
				if (Nickname.IsEmpty())
				{
					Nickname = TEXT("Unknown Player");
				}
				OutRankedUsers.Emplace(*NetId, Record->UserId, MoveTemp(Nickname), Record->Rank, Record->Score);
				OutProductUserIds.Add(Record->UserId);
			}

			EOS_Leaderboards_LeaderboardRecord_Release(Record);
		}
	}
}

/** Populate ReadObject rows from RankedUsers with all columns fetched via CopyLeaderboardUserScoreByUserId */
void PopulateRowsWithUserScores(EOS_HLeaderboards LeaderboardsHandle, 
	const TArray<FRankedUserInfo>& RankedUsers, 
	const FOnlineLeaderboardReadRef& ReadObject)
{
	char StatName[EOS_OSS_STRING_BUFFER_LENGTH];
	for (const FRankedUserInfo& Info : RankedUsers)
	{
		FOnlineStatsRow& Row = ReadObject->Rows.Emplace_GetRef(Info.Nickname, Info.NetId);
		Row.Rank = Info.Rank;

		EOS_Leaderboards_CopyLeaderboardUserScoreByUserIdOptions UserCopyOptions = { };
		UserCopyOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LEADERBOARDS_COPYLEADERBOARDUSERSCOREBYUSERID_API_LATEST, 1);
		UserCopyOptions.UserId = Info.ProductUserId;
		UserCopyOptions.StatName = StatName;

		for (const FColumnMetaData& Column : ReadObject->ColumnMetadata)
		{
			FCStringAnsi::Strncpy(StatName, TCHAR_TO_UTF8(*Column.ColumnName), EOS_OSS_STRING_BUFFER_LENGTH);
			EOS_Leaderboards_LeaderboardUserScore* UserScore = nullptr;
			EOS_EResult CopyResult = EOS_Leaderboards_CopyLeaderboardUserScoreByUserId(LeaderboardsHandle, &UserCopyOptions, &UserScore);
			if (CopyResult == EOS_EResult::EOS_Success)
			{
				Row.Columns.Emplace(Column.ColumnName, FVariantData(UserScore->Score));
				EOS_Leaderboards_LeaderboardUserScore_Release(UserScore);
			}
			else
			{
				Row.Columns.Emplace(Column.ColumnName, FVariantData());
			}
		}
	}
}

/** Populate ReadObject rows with empty columns (fallback when user scores query fails) */
void PopulateRowsWithEmptyColumns(const TArray<FRankedUserInfo>& RankedUsers, const FOnlineLeaderboardReadRef& ReadObject)
{
	for (const FRankedUserInfo& Info : RankedUsers)
	{
		FOnlineStatsRow& Row = ReadObject->Rows.Emplace_GetRef(Info.Nickname, Info.NetId);
		Row.Rank = Info.Rank;
		for (const FColumnMetaData& Column : ReadObject->ColumnMetadata)
		{
			Row.Columns.Emplace(Column.ColumnName, FVariantData());
		}
	}
}

/** Populate ReadObject rows with only the sorted column, using scores already collected in FRankedUserInfo */
void PopulateRowsSortedColumnOnly(const TArray<FRankedUserInfo>& RankedUsers, const FOnlineLeaderboardReadRef& ReadObject)
{
	for (const FRankedUserInfo& Info : RankedUsers)
	{
		FOnlineStatsRow& Row = ReadObject->Rows.Emplace_GetRef(Info.Nickname, Info.NetId);
		Row.Rank = Info.Rank;
		Row.Columns.Add(ReadObject->SortedColumn, FVariantData(Info.Score));
	}
}

/* Private */ }

struct FQueryLeaderboardForUserOptions :
	public EOS_Leaderboards_QueryLeaderboardUserScoresOptions
{
	TArray<char*> PointerArray;
	TArray<EOS_Leaderboards_UserScoresQueryStatInfo> StatInfoArray;
	TArray<EOS_ProductUserId> ProductUserIds;

	FQueryLeaderboardForUserOptions(const uint32 InNumColumns, TArray<EOS_ProductUserId>& InProductUserIds)
		: EOS_Leaderboards_QueryLeaderboardUserScoresOptions()
		, ProductUserIds(MoveTemp(InProductUserIds))
	{
		ApiVersion = 2;
		UE_EOS_CHECK_API_MISMATCH(EOS_LEADERBOARDS_QUERYLEADERBOARDUSERSCORES_API_LATEST, 2);

		StartTime = EOS_LEADERBOARDS_TIME_UNDEFINED;
		EndTime = EOS_LEADERBOARDS_TIME_UNDEFINED;

		PointerArray.AddZeroed(InNumColumns);
		for (int32 Index = 0; Index < PointerArray.Num(); Index++)
		{
			PointerArray[Index] = new char[EOS_OSS_STRING_BUFFER_LENGTH];
		}

		StatInfoArray.AddZeroed(InNumColumns);
		StatInfo = StatInfoArray.GetData();
		StatInfoCount = InNumColumns;

		UserIds = ProductUserIds.GetData();
		UserIdsCount = ProductUserIds.Num();
	}

	~FQueryLeaderboardForUserOptions()
	{
		for (int32 Index = 0; Index < PointerArray.Num(); Index++)
		{
			delete[] PointerArray[Index];
		}
	}
};

struct FQueryLeaderboardForUsersContext
{
	TArray<FUniqueNetIdRef> Players;
	FOnlineLeaderboardReadRef ReadObject;

	FQueryLeaderboardForUsersContext(const TArray<FUniqueNetIdRef>& InPlayers, FOnlineLeaderboardReadRef& InReadObject)
		: Players(InPlayers)
		, ReadObject(InReadObject)
	{
	}

	void AddEmptyRowForPlayer(FUniqueNetIdRef NetId, const FString& Nickname)
	{
		FOnlineStatsRow Row(Nickname, NetId);
		for (const FColumnMetaData& Column : ReadObject->ColumnMetadata)
		{
			Row.Columns.Add(Column.ColumnName, FVariantData());
		}
		Row.Rank = -1;
		ReadObject->Rows.Add(Row);
	}
};

typedef TEOSCallback<EOS_Leaderboards_OnQueryLeaderboardUserScoresCompleteCallback, EOS_Leaderboards_OnQueryLeaderboardUserScoresCompleteCallbackInfo, FOnlineLeaderboardsEOS> FQueryLeaderboardForUsersCallback;

bool FOnlineLeaderboardsEOS::ReadLeaderboards(const TArray<FUniqueNetIdRef>& Players, FOnlineLeaderboardReadRef& ReadObject)
{
	if (Players.Num() == 0)
	{
		EOSSubsystem->ExecuteNextTick([this]()
			{
				TriggerOnLeaderboardReadCompleteDelegates(false);
			});
		return true;
	}

	TArray<EOS_ProductUserId> ProductUserIds;
	ProductUserIds.Empty(Players.Num());
	// Validate the number of known users
	for (const FUniqueNetIdRef& NetId : Players)
	{
		const FUniqueNetIdEOS& EOSId = FUniqueNetIdEOS::Cast(*NetId);
		const EOS_ProductUserId UserId = EOSId.GetProductUserId();
		if (UserId != nullptr)
		{
			ProductUserIds.Add(UserId);
		}
	}

	FQueryLeaderboardForUserOptions Options(ReadObject->ColumnMetadata.Num(), ProductUserIds);
	Options.LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(0);
	// Convert the column names to stats
	int32 Index = 0;
	for (const FColumnMetaData& Column : ReadObject->ColumnMetadata)
	{
		FCStringAnsi::Strncpy(Options.PointerArray[Index], TCHAR_TO_UTF8(*Column.ColumnName), EOS_OSS_STRING_BUFFER_LENGTH);
		EOS_Leaderboards_UserScoresQueryStatInfo& StatInfo = Options.StatInfoArray[Index];
		StatInfo.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LEADERBOARDS_USERSCORESQUERYSTATINFO_API_LATEST, 1);
		StatInfo.StatName = Options.PointerArray[Index];
		StatInfo.Aggregation = EOS_ELeaderboardAggregation::EOS_LA_Latest;
		Index++;
	}

	TSharedPtr<FQueryLeaderboardForUsersContext> QueryContext = MakeShared<FQueryLeaderboardForUsersContext>(Players, ReadObject);

	FQueryLeaderboardForUsersCallback* CallbackObj = new FQueryLeaderboardForUsersCallback(FOnlineLeaderboardsEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, QueryContext](const EOS_Leaderboards_OnQueryLeaderboardUserScoresCompleteCallbackInfo* Data)
	{
		bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
		if (!bWasSuccessful)
		{
	
			UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("EOS_Leaderboards_QueryLeaderboardUserScores() failed with EOS result code (%s)"), *LexToString(Data->ResultCode));
			QueryContext->ReadObject->ReadState = EOnlineAsyncTaskState::Failed;
			TriggerOnLeaderboardReadCompleteDelegates(false);
			return;
		}

		char StatName[EOS_OSS_STRING_BUFFER_LENGTH];

		for (FUniqueNetIdRef NetId : QueryContext->Players)
		{
			FString Nickname = EOSSubsystem->UserManager->GetPlayerNickname(*NetId);
			const FUniqueNetIdEOS& EOSId = FUniqueNetIdEOS::Cast(*NetId);
			const EOS_ProductUserId UserId = EOSId.GetProductUserId();
			if (UserId == nullptr)
			{
				QueryContext->AddEmptyRowForPlayer(NetId, Nickname);
				continue;
			}

			FOnlineStatsRow Row(Nickname, NetId);

			EOS_Leaderboards_CopyLeaderboardUserScoreByUserIdOptions UserCopyOptions = { };
			UserCopyOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_LEADERBOARDS_COPYLEADERBOARDUSERSCOREBYUSERID_API_LATEST, 1);
			UserCopyOptions.UserId = UserId;
			UserCopyOptions.StatName = StatName;

			// Read each stat from the leaderboard
			for (const FColumnMetaData& Column : QueryContext->ReadObject->ColumnMetadata)
			{
				// Update which stat we are requesting
				FCStringAnsi::Strncpy(StatName, TCHAR_TO_UTF8(*Column.ColumnName), EOS_OSS_STRING_BUFFER_LENGTH);

				EOS_Leaderboards_LeaderboardUserScore* LeaderboardUserScore = nullptr;
				EOS_EResult UserCopyResult = EOS_Leaderboards_CopyLeaderboardUserScoreByUserId(EOSSubsystem->LeaderboardsHandle, &UserCopyOptions, &LeaderboardUserScore);
				if (UserCopyResult != EOS_EResult::EOS_Success)
				{
					Row.Columns.Add(Column.ColumnName, FVariantData());
					continue;
				}

				Row.Columns.Add(Column.ColumnName, FVariantData(LeaderboardUserScore->Score));

				EOS_Leaderboards_LeaderboardUserScore_Release(LeaderboardUserScore);
			}
			QueryContext->ReadObject->Rows.Add(Row);
		}

		// Manually build the ranks by sorting and then assigning rank values
		QueryContext->ReadObject->Rows.Sort([SortedColumn = QueryContext->ReadObject->SortedColumn](const FOnlineStatsRow& RowA, const FOnlineStatsRow& RowB)
		{
			const FVariantData& ValueA = RowA.Columns[SortedColumn];
			const FVariantData& ValueB = RowB.Columns[SortedColumn];
			if (ValueA.GetType() == ValueB.GetType())
			{
				int32 ScoreA = 0;
				int32 ScoreB = 0;
				ValueA.GetValue(ScoreA);
				ValueB.GetValue(ScoreB);
				return ScoreA >= ScoreB;
			}
			return true;
		});
		int32 Rank = 1;
		for (FOnlineStatsRow& Row : QueryContext->ReadObject->Rows)
		{
			Row.Rank = Rank++;
		}

		QueryContext->ReadObject->ReadState = EOnlineAsyncTaskState::Done;

		TriggerOnLeaderboardReadCompleteDelegates(true);
	};

	ReadObject->ReadState = EOnlineAsyncTaskState::InProgress;

	EOS_Leaderboards_QueryLeaderboardUserScores(EOSSubsystem->LeaderboardsHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());

	return true;
}

bool FOnlineLeaderboardsEOS::ReadLeaderboardsForFriends(int32 LocalUserNum, FOnlineLeaderboardReadRef& ReadObject)
{
	TArray<TSharedRef<FOnlineFriend>> Friends;
	EOSSubsystem->UserManager->GetFriendsList(LocalUserNum, FString(), Friends);
	if (Friends.Num() == 0)
	{
		EOSSubsystem->ExecuteNextTick([this]()
			{
				TriggerOnLeaderboardReadCompleteDelegates(false);
			});
		return true;
	}

	TArray<FUniqueNetIdRef> Players;
	Players.Reset(Friends.Num());

	for (TSharedRef<const FOnlineFriend> Friend : Friends)
	{
		Players.Add(Friend->GetUserId());
	}

	return ReadLeaderboards(Players, ReadObject);
}

typedef TEOSCallback<EOS_Leaderboards_OnQueryLeaderboardRanksCompleteCallback, EOS_Leaderboards_OnQueryLeaderboardRanksCompleteCallbackInfo, FOnlineLeaderboardsEOS> FQueryLeaderboardCallback;

bool FOnlineLeaderboardsEOS::ReadLeaderboardsAroundRank(int32 Rank, uint32 Range, FOnlineLeaderboardReadRef& ReadObject)
{
	if (Rank > EOS_MAX_NUM_RANKINGS)
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("ReadLeaderboardsAroundRank() - Rank (%d) must be <= 1000"), Rank);
		EOSSubsystem->ExecuteNextTick([this]()
			{
				TriggerOnLeaderboardReadCompleteDelegates(false);
			});
		return true;
	}

	// Calculate the range that we'll copy out of the full results
	uint32 StartIndex = (uint32)FMath::Clamp<int32>(Rank - (int32)Range, 0, EOS_MAX_NUM_RANKINGS);
	uint32 EndIndex = FMath::Clamp<uint32>(Rank + (int32)Range, 0, EOS_MAX_NUM_RANKINGS - 1);

	char LeaderboardId[EOS_OSS_STRING_BUFFER_LENGTH];
	EOS_Leaderboards_QueryLeaderboardRanksOptions Options = { };
	Options.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_LEADERBOARDS_QUERYLEADERBOARDRANKS_API_LATEST, 2);
	Options.LeaderboardId = LeaderboardId;
	Options.LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(0);
	FCStringAnsi::Strncpy(LeaderboardId, TCHAR_TO_UTF8(*ReadObject->LeaderboardName), EOS_OSS_STRING_BUFFER_LENGTH);

	FQueryLeaderboardCallback* CallbackObj = new FQueryLeaderboardCallback(FOnlineLeaderboardsEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, ReadObject, StartIndex, EndIndex](const EOS_Leaderboards_OnQueryLeaderboardRanksCompleteCallbackInfo* Data)
	{
		bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
		if (!bWasSuccessful)
		{
			UE_LOG_ONLINE_LEADERBOARD(Error, TEXT("EOS_Leaderboards_QueryLeaderboardRanks() failed with EOS result code (%s)"), *LexToString(Data->ResultCode));
			ReadObject->ReadState = EOnlineAsyncTaskState::Failed;
			TriggerOnLeaderboardReadCompleteDelegates(false);
			return;
		}
		EOS_Leaderboards_GetLeaderboardRecordCountOptions CountOptions = { };
		CountOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LEADERBOARDS_GETLEADERBOARDRECORDCOUNT_API_LATEST, 1);

		uint32 LeaderboardCount = EOS_Leaderboards_GetLeaderboardRecordCount(EOSSubsystem->LeaderboardsHandle, &CountOptions);
		// Handle fewer entries than our start index
		if (LeaderboardCount <= StartIndex)
		{
			ReadObject->ReadState = EOnlineAsyncTaskState::Done;
			TriggerOnLeaderboardReadCompleteDelegates(false);
			return;
		}
		// Handle fewer entries than our ending page index
		uint32 NewEndIndex = FMath::Clamp<uint32>(EndIndex, 0, LeaderboardCount - 1);

		// We get the list of users and resolve it first
		EOS_Leaderboards_CopyLeaderboardRecordByIndexOptions CopyOptions = { };
		CopyOptions.ApiVersion = 2;
		UE_EOS_CHECK_API_MISMATCH(EOS_LEADERBOARDS_COPYLEADERBOARDRECORDBYINDEX_API_LATEST, 2);

		TArray<EOS_ProductUserId> UsersToResolve;
		for (uint32 Index = StartIndex; Index <= NewEndIndex; Index++)
		{
			CopyOptions.LeaderboardRecordIndex = Index;

			EOS_Leaderboards_LeaderboardRecord* Record = nullptr;
			EOS_EResult Result = EOS_Leaderboards_CopyLeaderboardRecordByIndex(EOSSubsystem->LeaderboardsHandle, &CopyOptions, &Record);
			if (Result == EOS_EResult::EOS_Success)
			{
				UsersToResolve.Add(Record->UserId);
				EOS_Leaderboards_LeaderboardRecord_Release(Record);
			}
		}

		EOSSubsystem->UserManager->ResolveUniqueNetIds(0, UsersToResolve, [this, StartIndex, NewEndIndex, ReadObject]
		(const TMap<EOS_ProductUserId, FUniqueNetIdEOSRef>& ResolvedUniqueNetIds, const FOnlineError& Error)
		{
			TArray<Private::FRankedUserInfo> RankedUsers;
			TArray<EOS_ProductUserId> ProductUserIds;
			Private::CollectRankedUsers(EOSSubsystem->LeaderboardsHandle, StartIndex, NewEndIndex, ResolvedUniqueNetIds, RankedUsers, ProductUserIds);

			if (RankedUsers.IsEmpty())
			{
				ReadObject->ReadState = EOnlineAsyncTaskState::Done;
				TriggerOnLeaderboardReadCompleteDelegates(true);
				return;
			}

			// if there are extra columns beyond the sorted column, issue a follow-up QueryLeaderboardUserScores to populate all requested columns:
			const uint32 NumColumns = ReadObject->ColumnMetadata.Num();
			if (NumColumns > 1 || (NumColumns == 1 && ReadObject->ColumnMetadata[0].ColumnName != ReadObject->SortedColumn))
			{
				// TSharedPtr ensures cleanup even if TEOSCallback skips lambda execution (e.g., Owner destroyed)
				TSharedPtr<FQueryLeaderboardForUserOptions> UserScoresOptions = MakeShared<FQueryLeaderboardForUserOptions>(ReadObject->ColumnMetadata.Num(), ProductUserIds);
				UserScoresOptions->LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(0);
				int32 ColIdx = 0;
				for (const FColumnMetaData& Column : ReadObject->ColumnMetadata)
				{
					FCStringAnsi::Strncpy(UserScoresOptions->PointerArray[ColIdx], TCHAR_TO_UTF8(*Column.ColumnName), EOS_OSS_STRING_BUFFER_LENGTH);
					EOS_Leaderboards_UserScoresQueryStatInfo& StatInfo = UserScoresOptions->StatInfoArray[ColIdx];
					StatInfo.ApiVersion = 1;
					UE_EOS_CHECK_API_MISMATCH(EOS_LEADERBOARDS_USERSCORESQUERYSTATINFO_API_LATEST, 1);
					StatInfo.StatName = UserScoresOptions->PointerArray[ColIdx];
					// Aggregation matches ReadLeaderboards: if a leaderboard uses a different aggregation
					// (Min, Max, Sum), these values may differ from what QueryLeaderboardRanks used for ranking
					StatInfo.Aggregation = EOS_ELeaderboardAggregation::EOS_LA_Latest;
					++ColIdx;
				}

				FQueryLeaderboardForUsersCallback* UserScoresCallback = new FQueryLeaderboardForUsersCallback(FOnlineLeaderboardsEOSWeakPtr(AsShared()));
				UserScoresCallback->CallbackLambda = [this, ReadObject, RankedUsers = MoveTemp(RankedUsers), UserScoresOptions]
				(const EOS_Leaderboards_OnQueryLeaderboardUserScoresCompleteCallbackInfo* Data)
				{
					if (Data->ResultCode != EOS_EResult::EOS_Success)
					{
						UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("ReadLeaderboardsAroundRank: follow-up QueryLeaderboardUserScores failed (%s), falling back to empty columns"), *LexToString(Data->ResultCode));
						Private::PopulateRowsWithEmptyColumns(RankedUsers, ReadObject);
						ReadObject->ReadState = EOnlineAsyncTaskState::Done;
						TriggerOnLeaderboardReadCompleteDelegates(true);
						return;
					}

					Private::PopulateRowsWithUserScores(EOSSubsystem->LeaderboardsHandle, RankedUsers, ReadObject);
					ReadObject->ReadState = EOnlineAsyncTaskState::Done;
					TriggerOnLeaderboardReadCompleteDelegates(true);
				};

				EOS_Leaderboards_QueryLeaderboardUserScores(EOSSubsystem->LeaderboardsHandle, UserScoresOptions.Get(), UserScoresCallback, UserScoresCallback->GetCallbackPtr());
			}
			else
			{
				Private::PopulateRowsSortedColumnOnly(RankedUsers, ReadObject);
				ReadObject->ReadState = EOnlineAsyncTaskState::Done;
				TriggerOnLeaderboardReadCompleteDelegates(true);
			}
		});
	};

	ReadObject->ReadState = EOnlineAsyncTaskState::InProgress;

	EOS_Leaderboards_QueryLeaderboardRanks(EOSSubsystem->LeaderboardsHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());

	return true;
}

bool FOnlineLeaderboardsEOS::ReadLeaderboardsAroundUser(FUniqueNetIdRef Player, uint32 Range, FOnlineLeaderboardReadRef& ReadObject)
{
	TArray<FUniqueNetIdRef> Players;
	Players.Add(Player);

	return ReadLeaderboards(Players, ReadObject);
}

void FOnlineLeaderboardsEOS::FreeStats(FOnlineLeaderboardRead& ReadObject)
{
	// Nothing needed to be done here
}

bool FOnlineLeaderboardsEOS::WriteLeaderboards(const FName& SessionName, const FUniqueNetId& Player, FOnlineLeaderboardWrite& WriteObject)
{
	TArray<FOnlineStatsUserUpdatedStats> StatsToWrite;

	FOnlineStatsUserUpdatedStats& UpdatedStats = StatsToWrite.Emplace_GetRef(Player.AsShared());
	for (const TPair<FString, FVariantData>& Stat : WriteObject.Properties)
	{
		UpdatedStats.Stats.Add(Stat.Key, FOnlineStatUpdate(Stat.Value, FOnlineStatUpdate::EOnlineStatModificationType::Unknown));
	}

	EOSSubsystem->StatsInterfacePtr->UpdateStats(Player.AsShared(), StatsToWrite, FOnlineStatsUpdateStatsComplete());
	return true;
}

bool FOnlineLeaderboardsEOS::FlushLeaderboards(const FName& SessionName)
{
	UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("FlushLeaderboards() is not supported"));

	EOSSubsystem->ExecuteNextTick([this, SessionName]()
		{
			TriggerOnLeaderboardFlushCompleteDelegates(SessionName, false);
		});
	
	return true;
}

bool FOnlineLeaderboardsEOS::WriteOnlinePlayerRatings(const FName& SessionName, int32 LeaderboardId, const TArray<FOnlinePlayerScore>& PlayerScores)
{
	return false;
}

bool FOnlineLeaderboardsEOS::HandleLeaderboardsExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
#if !UE_BUILD_SHIPPING
	bool bWasHandled = true;

	if (FParse::Command(&Cmd, TEXT("ReadLeaderboards"))) /* ONLINE LEADERBOARDS ReadLeaderboards LocalUserNum=0 LeaderboardName=Deaths SortedColumn=Deaths DataType=Int32 */
	{
		int LocalUserNum;
		FParse::Value(Cmd, TEXT("LocalUserNum="), LocalUserNum);
		const FUniqueNetIdPtr NetIdPtr = EOSSubsystem->UserManager->GetUniquePlayerId(LocalUserNum);

		FString LeaderboardName;
		FParse::Value(Cmd, TEXT("LeaderboardName="), LeaderboardName);
		FString SortedColumn;
		FParse::Value(Cmd, TEXT("SortedColumn="), SortedColumn);
		FString DataTypeStr;
		FParse::Value(Cmd, TEXT("DataType="), DataTypeStr);
		const EOnlineKeyValuePairDataType::Type DataType = EOnlineKeyValuePairDataType::FromString(DataTypeStr);

		FOnlineLeaderboardReadRef ReadRef = MakeShared<FOnlineLeaderboardRead>();
		ReadRef->LeaderboardName = LeaderboardName;
		ReadRef->SortedColumn = SortedColumn;
		ReadRef->ColumnMetadata.Add(FColumnMetaData(SortedColumn, DataType));

		AddOnLeaderboardReadCompleteDelegate_Handle(FOnLeaderboardReadCompleteDelegate::CreateLambda([ReadRef](bool bWasSuccessful)
			{
				UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("FOnlineLeaderboardsEOS::ReadLeaderboards finished with bWasSuccessful=%s"), *LexToString(bWasSuccessful));
				UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("%s"), *ReadRef->ToLogString());
			}));

		ReadLeaderboards({ NetIdPtr.ToSharedRef() }, ReadRef);
	}
	else if (FParse::Command(&Cmd, TEXT("ReadLeaderboardsForFriends"))) /* ONLINE LEADERBOARDS ReadLeaderboardsForFriends LocalUserNum=0 LeaderboardName=Deaths SortedColumn=Deaths DataType=Int32 */
	{
		int LocalUserNum;
		FParse::Value(Cmd, TEXT("LocalUserNum="), LocalUserNum);

		FString LeaderboardName;
		FParse::Value(Cmd, TEXT("LeaderboardName="), LeaderboardName);
		FString SortedColumn;
		FParse::Value(Cmd, TEXT("SortedColumn="), SortedColumn);
		FString DataTypeStr;
		FParse::Value(Cmd, TEXT("DataType="), DataTypeStr);
		const EOnlineKeyValuePairDataType::Type DataType = EOnlineKeyValuePairDataType::FromString(DataTypeStr);

		FOnlineLeaderboardReadRef ReadRef = MakeShared<FOnlineLeaderboardRead>();
		ReadRef->LeaderboardName = LeaderboardName;
		ReadRef->SortedColumn = SortedColumn;
		ReadRef->ColumnMetadata.Add(FColumnMetaData(SortedColumn, DataType));

		AddOnLeaderboardReadCompleteDelegate_Handle(FOnLeaderboardReadCompleteDelegate::CreateLambda([ReadRef](bool bWasSuccessful)
			{
				UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("FOnlineLeaderboardsEOS::ReadLeaderboards finished with bWasSuccessful=%s"), *LexToString(bWasSuccessful));
				UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("%s"), *ReadRef->ToLogString());
			}));

		ReadLeaderboardsForFriends(LocalUserNum, ReadRef);
	}
	else if (FParse::Command(&Cmd, TEXT("ReadLeaderboardsAroundRank"))) /* ONLINE LEADERBOARDS ReadLeaderboardsAroundRank Rank=1 Range=10 LeaderboardName=Deaths SortedColumn=Deaths DataType=Int32 ExtraColumns=Kills,Wins */
	{
		int Rank;
		FParse::Value(Cmd, TEXT("Rank="), Rank);
		int Range;
		FParse::Value(Cmd, TEXT("Range="), Range);

		FString LeaderboardName;
		FParse::Value(Cmd, TEXT("LeaderboardName="), LeaderboardName);
		FString SortedColumn;
		FParse::Value(Cmd, TEXT("SortedColumn="), SortedColumn);
		FString DataTypeStr;
		FParse::Value(Cmd, TEXT("DataType="), DataTypeStr);
		const EOnlineKeyValuePairDataType::Type DataType = EOnlineKeyValuePairDataType::FromString(DataTypeStr);

		FOnlineLeaderboardReadRef ReadRef = MakeShared<FOnlineLeaderboardRead>();
		ReadRef->LeaderboardName = LeaderboardName;
		ReadRef->SortedColumn = SortedColumn;
		ReadRef->ColumnMetadata.Add(FColumnMetaData(SortedColumn, DataType));

		// Parse optional extra columns (comma-separated) to test multi-column support
		FString ExtraColumnsStr;
		if (FParse::Value(Cmd, TEXT("ExtraColumns="), ExtraColumnsStr))
		{
			TArray<FString> ExtraColumns;
			ExtraColumnsStr.ParseIntoArray(ExtraColumns, TEXT(","), true);
			for (const FString& ExtraColumn : ExtraColumns)
			{
				ReadRef->ColumnMetadata.Add(FColumnMetaData(ExtraColumn, DataType));
			}
		}

		AddOnLeaderboardReadCompleteDelegate_Handle(FOnLeaderboardReadCompleteDelegate::CreateLambda([ReadRef](bool bWasSuccessful)
		{
			UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("FOnlineLeaderboardsEOS::ReadLeaderboardsAroundRank finished with bWasSuccessful=%s"), *LexToString(bWasSuccessful));
			UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("%s"), *ReadRef->ToLogString());
			for (const FOnlineStatsRow& Row : ReadRef->Rows)
			{
				UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("  Rank=%d Player=%s Columns=%d"), Row.Rank, *Row.NickName, Row.Columns.Num());
				for (const TTuple<FString, FVariantData>& ColumnPair : Row.Columns)
				{
					UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("    %s = %s"), *ColumnPair.Key, *ColumnPair.Value.ToString());
				}
			}
		}));

		ReadLeaderboardsAroundRank(Rank, Range, ReadRef);
	}
	else if (FParse::Command(&Cmd, TEXT("ReadLeaderboardsAroundUser"))) /* ONLINE LEADERBOARDS ReadLeaderboardsAroundUser LocalUserNum=0 Range=10 LeaderboardName=Deaths SortedColumn=Deaths DataType=Int32 */
	{
		int LocalUserNum;
		FParse::Value(Cmd, TEXT("LocalUserNum="), LocalUserNum);
		const FUniqueNetIdPtr NetIdPtr = EOSSubsystem->UserManager->GetUniquePlayerId(LocalUserNum);

		int Range;
		FParse::Value(Cmd, TEXT("Range="), Range);

		FString LeaderboardName;
		FParse::Value(Cmd, TEXT("LeaderboardName="), LeaderboardName);
		FString SortedColumn;
		FParse::Value(Cmd, TEXT("SortedColumn="), SortedColumn);
		FString DataTypeStr;
		FParse::Value(Cmd, TEXT("DataType="), DataTypeStr);
		const EOnlineKeyValuePairDataType::Type DataType = EOnlineKeyValuePairDataType::FromString(DataTypeStr);

		FOnlineLeaderboardReadRef ReadRef = MakeShared<FOnlineLeaderboardRead>();
		ReadRef->LeaderboardName = LeaderboardName;
		ReadRef->SortedColumn = SortedColumn;
		ReadRef->ColumnMetadata.Add(FColumnMetaData(SortedColumn, DataType));

		AddOnLeaderboardReadCompleteDelegate_Handle(FOnLeaderboardReadCompleteDelegate::CreateLambda([ReadRef](bool bWasSuccessful)
			{
				UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("FOnlineLeaderboardsEOS::ReadLeaderboards finished with bWasSuccessful=%s"), *LexToString(bWasSuccessful));
				UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("%s"), *ReadRef->ToLogString());
			}));

		ReadLeaderboardsAroundUser(NetIdPtr.ToSharedRef(), Range, ReadRef);
	}
	else
	{
		bWasHandled = false;
	}

	return bWasHandled;
#else
	return false;
#endif // !UE_BUILD_SHIPPING
}

#endif
