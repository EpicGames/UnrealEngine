// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineLeaderboardInterface.h"
#include "OnlineSubsystemEOSTypes.h"

class FOnlineSubsystemEOS;

#if WITH_EOS_SDK
#include "eos_leaderboards_types.h"

#define EOS_MAX_NUM_RANKINGS 1000

/**
 * Interface for interacting with EOS stats
 */
class FOnlineLeaderboardsEOS
	: public IOnlineLeaderboards
	, public TSharedFromThis<FOnlineLeaderboardsEOS, ESPMode::ThreadSafe>
{
public:
	FOnlineLeaderboardsEOS() = delete;
	virtual ~FOnlineLeaderboardsEOS() = default;

// IOnlineLeaderboards Interface
	virtual bool ReadLeaderboards(const TArray< FUniqueNetIdRef >& Players, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual bool ReadLeaderboardsForFriends(int32 LocalUserNum, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual bool ReadLeaderboardsAroundRank(int32 Rank, uint32 Range, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual bool ReadLeaderboardsAroundUser(FUniqueNetIdRef Player, uint32 Range, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual void FreeStats(FOnlineLeaderboardRead& ReadObject) override;
	virtual bool WriteLeaderboards(const FName& SessionName, const FUniqueNetId& Player, FOnlineLeaderboardWrite& WriteObject) override;
	virtual bool FlushLeaderboards(const FName& SessionName) override;
	virtual bool WriteOnlinePlayerRatings(const FName& SessionName, int32 LeaderboardId, const TArray<FOnlinePlayerScore>& PlayerScores) override;
// ~IOnlineLeaderboards Interface

	FOnlineLeaderboardsEOS(FOnlineSubsystemEOS* InSubsystem)
		: EOSSubsystem(InSubsystem)
	{
	}

	bool HandleLeaderboardsExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);

private:

	/** Reference to the main EOS subsystem */
	FOnlineSubsystemEOS* EOSSubsystem;
};

typedef TSharedPtr<FOnlineLeaderboardsEOS, ESPMode::ThreadSafe> FOnlineLeaderboardsEOSPtr;
typedef TWeakPtr<FOnlineLeaderboardsEOS, ESPMode::ThreadSafe> FOnlineLeaderboardsEOSWeakPtr;

#endif
