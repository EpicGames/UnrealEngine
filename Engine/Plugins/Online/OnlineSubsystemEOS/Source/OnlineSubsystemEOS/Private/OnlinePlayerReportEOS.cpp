// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlinePlayerReportEOS.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemEOS.h"
#include "OnlineSubsystemEOSPrivate.h"

#if WITH_EOS_SDK
#include "eos_reports.h"
#include "eos_reports_types.h"
#include "eos_sdk.h"

DEFINE_LOG_CATEGORY(LogOnlinePlayerReportEOS); 

FOnlinePlayerReportEOS::FOnlinePlayerReportEOS(FOnlineSubsystemEOS* InSubsystem)
	: EOSSubsystem(InSubsystem)
{
}

// Convert the plugin player report category enum to the EOS SDK enum
EOS_EPlayerReportsCategory ToEOSPlayerReportsCategory(IOnlinePlayerReportEOS::EPlayerReportCategory Category)
{
	switch (Category)
	{
	case IOnlinePlayerReportEOS::EPlayerReportCategory::Cheating:
	{
		return EOS_EPlayerReportsCategory::EOS_PRC_Cheating;
	}
	case IOnlinePlayerReportEOS::EPlayerReportCategory::Exploiting:
	{
		return EOS_EPlayerReportsCategory::EOS_PRC_Exploiting;
	}
	case IOnlinePlayerReportEOS::EPlayerReportCategory::OffensiveProfile:
	{
		return EOS_EPlayerReportsCategory::EOS_PRC_OffensiveProfile;
	}
	case IOnlinePlayerReportEOS::EPlayerReportCategory::VerbalAbuse:
	{
		return EOS_EPlayerReportsCategory::EOS_PRC_VerbalAbuse;
	}
	case IOnlinePlayerReportEOS::EPlayerReportCategory::Scamming:
	{
		return EOS_EPlayerReportsCategory::EOS_PRC_Scamming;
	}
	case IOnlinePlayerReportEOS::EPlayerReportCategory::Spamming:
	{
		return EOS_EPlayerReportsCategory::EOS_PRC_Spamming;
	}
	}
	// Default to other
	return EOS_EPlayerReportsCategory::EOS_PRC_Other;
}

typedef TEOSCallback<EOS_Reports_OnSendPlayerBehaviorReportCompleteCallback, EOS_Reports_SendPlayerBehaviorReportCompleteCallbackInfo, FOnlinePlayerReportEOS> FSendPlayerReportCallback;

void FOnlinePlayerReportEOS::SendPlayerReport(const FUniqueNetId& LocalUserId, const FUniqueNetId& TargetUserId, FSendPlayerReportSettings&& SendPlayerReportSettings, FOnSendPlayerReportComplete&& Delegate)
{
	const FUniqueNetIdEOS& LocalEOSId = FUniqueNetIdEOS::Cast(LocalUserId);
	const EOS_ProductUserId LocalProductUserId = LocalEOSId.GetProductUserId();

	const FUniqueNetIdEOS& TargetEOSId = FUniqueNetIdEOS::Cast(TargetUserId);
	const EOS_ProductUserId TargetProductUserId = TargetEOSId.GetProductUserId();

	EOS_Reports_SendPlayerBehaviorReportOptions Options = {};
	Options.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_REPORTS_SENDPLAYERBEHAVIORREPORT_API_LATEST, 2);
	Options.ReporterUserId = LocalProductUserId;
    Options.ReportedUserId = TargetProductUserId;
	Options.Category = ToEOSPlayerReportsCategory(SendPlayerReportSettings.Category);
	const auto MessagedUTF8 = StringCast<UTF8CHAR>(*SendPlayerReportSettings.Message);
	const auto ContextUTF8 = StringCast<UTF8CHAR>(*SendPlayerReportSettings.Context);
	Options.Message = SendPlayerReportSettings.Message.IsEmpty() ? nullptr : (const char*)MessagedUTF8.Get();
	Options.Context = SendPlayerReportSettings.Context.IsEmpty() ? nullptr : (const char*)ContextUTF8.Get();
	
	FSendPlayerReportCallback* CallbackObj = new FSendPlayerReportCallback(AsWeak());
	CallbackObj->CallbackLambda = [LocalProductUserId, TargetProductUserId, Delegate = MoveTemp(Delegate)](const EOS_Reports_SendPlayerBehaviorReportCompleteCallbackInfo* Data)
		{
			const bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;

			if (!bWasSuccessful)
			{
				UE_LOG_ONLINE_PLAYERREPORTEOS(Warning, TEXT("LocalUserId (%s) failed to send the player report for TargetUserId(%s). The error code is: (%s)."),*LexToString(LocalProductUserId), *LexToString(TargetProductUserId), *LexToString(Data->ResultCode));
			}
			Delegate.ExecuteIfBound(bWasSuccessful);
		};

	EOS_Reports_SendPlayerBehaviorReport(EOSSubsystem->PlayerReportHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
}
#endif
