// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/ScopeLock.h"
#include "Interfaces/OnlinePlayerReportEOSInterface.h"
#include "OnlineSubsystemEOSTypes.h"

DECLARE_LOG_CATEGORY_EXTERN(LogOnlinePlayerReportEOS, Log, All);

#define UE_LOG_ONLINE_PLAYERREPORTEOS(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlinePlayerReportEOS, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

class FOnlineSubsystemEOS;

#if WITH_EOS_SDK

/**
 * Interface for interacting with EOS player reports
 */
class FOnlinePlayerReportEOS
	: public IOnlinePlayerReportEOS
	, public TSharedFromThis<FOnlinePlayerReportEOS, ESPMode::ThreadSafe>
{
public:
	FOnlinePlayerReportEOS() = delete;
	virtual ~FOnlinePlayerReportEOS() = default;

	FOnlinePlayerReportEOS(FOnlineSubsystemEOS* InSubsystem);

	virtual void SendPlayerReport(const FUniqueNetId& LocalUserId, const FUniqueNetId& TargetUserId, FSendPlayerReportSettings&& SendPlayerReportSettings, FOnSendPlayerReportComplete&& Delegate) override;

private:
	/** Reference to the main EOS subsystem */
	FOnlineSubsystemEOS* EOSSubsystem;
};
#endif