// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlinePlayerSanctionEOS.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemEOS.h"
#include "OnlineSubsystemEOSPrivate.h"

#if WITH_EOS_SDK
#include "EOSShared.h"
#include "eos_sanctions.h"
#include "eos_sanctions_types.h"
#include "eos_sdk.h"

DEFINE_LOG_CATEGORY(LogOnlinePlayerSanctionEOS);

FOnlinePlayerSanctionEOS::FOnlinePlayerSanctionEOS(FOnlineSubsystemEOS* InSubsystem)
	: EOSSubsystem(InSubsystem)
{
}

// Function to convert UE enum to EOS SDK enum
EOS_ESanctionAppealReason ToEOSSanctionAppealReason(IOnlinePlayerSanctionEOS::EPlayerSanctionAppealReason Reason)
{
	switch (Reason)
	{
	case IOnlinePlayerSanctionEOS::EPlayerSanctionAppealReason::IncorrectSanction:
		return EOS_ESanctionAppealReason::EOS_SAR_IncorrectSanction;
	case IOnlinePlayerSanctionEOS::EPlayerSanctionAppealReason::CompromisedAccount:
		return EOS_ESanctionAppealReason::EOS_SAR_CompromisedAccount;
	case IOnlinePlayerSanctionEOS::EPlayerSanctionAppealReason::UnfairPunishment:
		return EOS_ESanctionAppealReason::EOS_SAR_UnfairPunishment;
	case IOnlinePlayerSanctionEOS::EPlayerSanctionAppealReason::AppealForForgivenesss:
		return EOS_ESanctionAppealReason::EOS_SAR_AppealForForgiveness; 
	}
	return EOS_ESanctionAppealReason::EOS_SAR_Invalid; 
}

typedef TEOSCallback<EOS_Sanctions_CreatePlayerSanctionAppealCallback, EOS_Sanctions_CreatePlayerSanctionAppealCallbackInfo, FOnlinePlayerSanctionEOS> FCreatePlayerSanctionAppealCallback;

void FOnlinePlayerSanctionEOS::CreatePlayerSanctionAppeal(const FUniqueNetId& LocalUserId, FPlayerSanctionAppealSettings&& SanctionAppealSettings, FOnCreatePlayerSanctionAppealComplete&& Delegate)
{

	const FUniqueNetIdEOS& LocalEOSId = FUniqueNetIdEOS::Cast(LocalUserId);
	const EOS_ProductUserId LocalProductUserId = LocalEOSId.GetProductUserId();

	EOS_Sanctions_CreatePlayerSanctionAppealOptions Options = {};
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SANCTIONS_CREATEPLAYERSANCTIONAPPEAL_API_LATEST, 1);
	Options.LocalUserId = LocalProductUserId; 
	Options.Reason = ToEOSSanctionAppealReason(SanctionAppealSettings.Reason);
	const auto ReferenceIdUTF8 = StringCast<UTF8CHAR>(*SanctionAppealSettings.ReferenceId);
	Options.ReferenceId = (const char*)ReferenceIdUTF8.Get();

	FCreatePlayerSanctionAppealCallback* CallbackObj = new FCreatePlayerSanctionAppealCallback(AsWeak());
	CallbackObj->CallbackLambda = [Delegate = MoveTemp(Delegate)](const EOS_Sanctions_CreatePlayerSanctionAppealCallbackInfo* Data)
		{
			const bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;

			if (!bWasSuccessful)
			{
				UE_LOG_ONLINE_PLAYERSANCTIONEOS(Warning, TEXT("Failed to send sanction appeal for local user."));
			}
			Delegate.ExecuteIfBound(bWasSuccessful); 
		};

	EOS_Sanctions_CreatePlayerSanctionAppeal(EOSSubsystem->PlayerSanctionHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr()); 
}

typedef TEOSCallback<EOS_Sanctions_OnQueryActivePlayerSanctionsCallback, EOS_Sanctions_QueryActivePlayerSanctionsCallbackInfo, FOnlinePlayerSanctionEOS> FQueryActivePlayerSanctionsCallback;

void FOnlinePlayerSanctionEOS::QueryActivePlayerSanctions(const FUniqueNetId& LocalUserId, const FUniqueNetId& TargetUserId, FOnQueryActivePlayerSanctionsComplete&& Delegate)
{
	const FUniqueNetIdEOS& LocalEOSId = FUniqueNetIdEOS::Cast(LocalUserId);
	const EOS_ProductUserId LocalProductUserId = LocalEOSId.GetProductUserId();

	const FUniqueNetIdEOS& TargetEOSId = FUniqueNetIdEOS::Cast(TargetUserId);
	const EOS_ProductUserId TargetProductUserId = TargetEOSId.GetProductUserId();

	EOS_Sanctions_QueryActivePlayerSanctionsOptions QueryOptions = {}; 
	QueryOptions.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_SANCTIONS_QUERYACTIVEPLAYERSANCTIONS_API_LATEST, 2);
	QueryOptions.LocalUserId = LocalProductUserId; 
	QueryOptions.TargetUserId = TargetProductUserId; 

	FQueryActivePlayerSanctionsCallback* CallbackObj = new FQueryActivePlayerSanctionsCallback(AsWeak());
	CallbackObj->CallbackLambda = [this, LambdaPlayerId = TargetUserId.AsShared(), Delegate = MoveTemp(Delegate)](const  EOS_Sanctions_QueryActivePlayerSanctionsCallbackInfo* Data)
		{
			const bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;

			if (bWasSuccessful)
			{
				TArray<FOnlinePlayerSanction>& PlayerSanctions = CachedPlayerSanctionsMap.Emplace(LambdaPlayerId);

				const FUniqueNetIdEOS& TargetEOSId = FUniqueNetIdEOS::Cast(*LambdaPlayerId);
				const EOS_ProductUserId TargetProductUserId = TargetEOSId.GetProductUserId();

				EOS_Sanctions_GetPlayerSanctionCountOptions CountOptions = {};
				CountOptions.ApiVersion = 1; 
				UE_EOS_CHECK_API_MISMATCH(EOS_SANCTIONS_GETPLAYERSANCTIONCOUNT_API_LATEST, 1);
				CountOptions.TargetUserId = TargetProductUserId;

				uint32 SanctionCount = EOS_Sanctions_GetPlayerSanctionCount(EOSSubsystem->PlayerSanctionHandle, &CountOptions); 

				EOS_Sanctions_CopyPlayerSanctionByIndexOptions CopyOptions = {};
				CopyOptions.ApiVersion = 1; 
				UE_EOS_CHECK_API_MISMATCH(EOS_SANCTIONS_COPYPLAYERSANCTIONBYINDEX_API_LATEST, 1);
				CopyOptions.TargetUserId = TargetProductUserId;

				for (uint32 SanctionIndex = 0; SanctionIndex < SanctionCount; SanctionIndex++)
				{
					CopyOptions.SanctionIndex = SanctionIndex;

					EOS_Sanctions_PlayerSanction* PlayerSanctionEOS = nullptr; 

					EOS_EResult Result = EOS_Sanctions_CopyPlayerSanctionByIndex(EOSSubsystem->PlayerSanctionHandle, &CopyOptions, &PlayerSanctionEOS);
					if (Result == EOS_EResult::EOS_Success)
					{
						FOnlinePlayerSanction& PlayerSanction = PlayerSanctions.Emplace_GetRef();
						PlayerSanction.TimePlaced = PlayerSanctionEOS->TimePlaced;
						PlayerSanction.TimeExpires = PlayerSanctionEOS->TimeExpires;
						PlayerSanction.ReferenceId = PlayerSanctionEOS->ReferenceId;
						PlayerSanction.Action = PlayerSanctionEOS->Action;

						EOS_Sanctions_PlayerSanction_Release(PlayerSanctionEOS); 

					}
					else
					{
						UE_LOG_ONLINE_PLAYERSANCTIONEOS(Warning, TEXT("EOS_Sanctions_CopyPlayerSanctionByIndex() failed for player (%s). The error code is: (%s)"), *LambdaPlayerId->ToString(),*LexToString(Result));
					}
				}
			}
			else
			{
				UE_LOG_ONLINE_PLAYERSANCTIONEOS(Warning, TEXT("Failed to query active player sanctions for player (%s)."), *LambdaPlayerId->ToString());
			}
			Delegate.ExecuteIfBound(bWasSuccessful);
		};

	EOS_Sanctions_QueryActivePlayerSanctions(EOSSubsystem->PlayerSanctionHandle, &QueryOptions, CallbackObj, CallbackObj->GetCallbackPtr()); 
}
EOnlineCachedResult::Type FOnlinePlayerSanctionEOS::GetCachedActivePlayerSanctions(const FUniqueNetId& TargetUserId, TArray<FOnlinePlayerSanction>& OutPlayerSanctions)
{
	if (const TArray<FOnlinePlayerSanction>* CachedPlayerSanctions = CachedPlayerSanctionsMap.Find(TargetUserId.AsShared()))
	{
		OutPlayerSanctions = *CachedPlayerSanctions;
		return EOnlineCachedResult::Success;
	}
	return EOnlineCachedResult::NotFound;
}
#endif