// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/ScopeLock.h"
#include "Interfaces/OnlinePlayerSanctionEOSInterface.h"
#include "OnlineSubsystemEOSTypes.h"

DECLARE_LOG_CATEGORY_EXTERN(LogOnlinePlayerSanctionEOS, Log, All);

#define UE_LOG_ONLINE_PLAYERSANCTIONEOS(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOnlinePlayerSanctionEOS, Verbosity, "%ls%ls", ONLINE_LOG_PREFIX, *FString::Printf(Format, ##__VA_ARGS__)); \
}

class FOnlineSubsystemEOS;

#if WITH_EOS_SDK

/**
 * Interface for interacting with EOS sanctions
 */
class FOnlinePlayerSanctionEOS
	: public IOnlinePlayerSanctionEOS
	, public TSharedFromThis<FOnlinePlayerSanctionEOS, ESPMode::ThreadSafe>
{
public:
	FOnlinePlayerSanctionEOS() = delete;
	virtual ~FOnlinePlayerSanctionEOS() = default;

	FOnlinePlayerSanctionEOS(FOnlineSubsystemEOS* InSubsystem);

	virtual void CreatePlayerSanctionAppeal(const FUniqueNetId& LocalUserId, FPlayerSanctionAppealSettings&& SanctionAppealSettings, FOnCreatePlayerSanctionAppealComplete&& Delegate) override;

	virtual void QueryActivePlayerSanctions(const FUniqueNetId& LocalUserId, const FUniqueNetId& TargetUserId, FOnQueryActivePlayerSanctionsComplete&& Delegate) override;

	virtual  EOnlineCachedResult::Type GetCachedActivePlayerSanctions(const FUniqueNetId& TargetUserId, TArray<FOnlinePlayerSanction>& OutPlayerSanctions) override;


private:
	/** Reference to the main EOS subsystem */
	FOnlineSubsystemEOS* EOSSubsystem;

	/** Holds the cached info from the last time this was called */
	TUniqueNetIdMap<TArray<FOnlinePlayerSanction>> CachedPlayerSanctionsMap;
};

#endif
