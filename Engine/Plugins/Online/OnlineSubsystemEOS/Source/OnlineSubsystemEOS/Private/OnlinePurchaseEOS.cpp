// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlinePurchaseEOS.h"
#include "OnlineEntitlementsEOS.h"
#include "EOSSettings.h"
#include "IEOSSDKManager.h"

#if WITH_EOS_SDK
#include "OnlineSubsystemEOS.h"
#include "OnlineSubsystemEOSPrivate.h"
#include "UserManagerEOS.h"
#include "eos_ecom.h"

#define ONLINE_ERROR_NAMESPACE "com.epicgames.oss.eos.error"


FOnlinePurchaseEOS::FOnlinePurchaseEOS(FOnlineSubsystemEOS* InSubsystem)
	: EOSSubsystem(InSubsystem)
{
	check(EOSSubsystem != nullptr);
}

typedef TEOSCallback<EOS_Ecom_OnCheckoutCallback, EOS_Ecom_CheckoutCallbackInfo, FOnlinePurchaseEOS> FPurchaseCheckoutCallback;

void FOnlinePurchaseEOS::Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseCheckoutComplete& Delegate)
{
	const FUniqueNetIdEOS& UserEOSId = FUniqueNetIdEOS::Cast(UserId);
	const EOS_EpicAccountId AccountId = UserEOSId.GetEpicAccountId();
	if (AccountId == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("[%hs] EpicAccountId is null"), __FUNCTION__);
		Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::InvalidUser), MakeShared<FPurchaseReceipt>());
		return;
	}
	if (CheckoutRequest.PurchaseOffers.Num() == 0)
	{
		UE_LOG_ONLINE(Error, TEXT("[%hs] PurchaseOffers.Num() is 0"), __FUNCTION__);
		Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::InvalidParams), MakeShared<FPurchaseReceipt>());
		return;
	}
	if (CheckoutRequest.PurchaseOffers.Num() > EOS_ECOM_CHECKOUT_MAX_ENTRIES)
	{
		UE_LOG_ONLINE(Error, TEXT("[%hs] PurcahseOffer.Num() over maximum of %d "), __FUNCTION__, EOS_ECOM_CHECKOUT_MAX_ENTRIES);
		Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::InvalidParams), MakeShared<FPurchaseReceipt>());
		return;
	}

	const int32 NumItems = CheckoutRequest.PurchaseOffers.Num();
	TArray<EOS_Ecom_CheckoutEntry> Entries;
	TArray<FTCHARToUTF8> OfferIdsUtf8;
	Entries.AddZeroed(NumItems);
	OfferIdsUtf8.Reserve(NumItems);

	for (int32 Index = 0; Index < NumItems; Index++)
	{
		OfferIdsUtf8.Emplace(*CheckoutRequest.PurchaseOffers[Index].OfferId);
		Entries[Index].ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_CHECKOUTENTRY_API_LATEST, 1);
		Entries[Index].OfferId = OfferIdsUtf8.Last().Get();
	}

	EOS_Ecom_CheckoutOptions Options = { };
	Options.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_CHECKOUT_API_LATEST, 2);
	Options.LocalUserId = AccountId;
	Options.EntryCount = NumItems;
	Options.Entries = (const EOS_Ecom_CheckoutEntry*)Entries.GetData();
	Options.OverrideCatalogNamespace = nullptr;
	Options.PreferredOrientation = EOS_ECheckoutOrientation::EOS_ECO_Default;

	FPurchaseCheckoutCallback* CallbackObj = new FPurchaseCheckoutCallback(FOnlinePurchaseEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, OnComplete = FOnPurchaseCheckoutComplete(Delegate)](const EOS_Ecom_CheckoutCallbackInfo* Data)
	{
		const EOS_EResult CheckoutResult = Data->ResultCode;
		if (CheckoutResult != EOS_EResult::EOS_Success)
		{
			if (CheckoutResult == EOS_EResult::EOS_Canceled)
			{
				UE_LOG_ONLINE(Log, TEXT("[FOnlinePurchaseEOS::Checkout] EOS_Ecom_Checkout was canceled by user"));
				OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Canceled), MakeShared<FPurchaseReceipt>());
			}
			else if (CheckoutResult == EOS_EResult::EOS_Ecom_PurchaseProcessing)
			{
				// See header documentation for EOS_Ecom_PurchaseProcessing and online EOS documentation for more information
				UE_LOG_ONLINE(Log, TEXT("[FOnlinePurchaseEOS::Checkout] QueryEntitlements will need to be called to confirm the purchase. This can be done via UI button"));
				OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::FailExtended, LexToString(CheckoutResult)), MakeShared<FPurchaseReceipt>());
			}
			else
			{
				UE_LOG_ONLINE(Error, TEXT("[FOnlinePurchaseEOS::Checkout] EOS_Ecom_Checkout: failed with error (%s)"), *LexToString(CheckoutResult));
				OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Unknown), MakeShared<FPurchaseReceipt>());
			}
			return;
		}

		TSharedRef<FPurchaseReceipt> PurchaseReceipt = MakeShared<FPurchaseReceipt>();
		PurchaseReceipt->TransactionId = Data->TransactionId ? StringCast<TCHAR>(Data->TransactionId).Get() : FString();
		PurchaseReceipt->TransactionState = EPurchaseTransactionState::Purchased;

		EOS_Ecom_CopyTransactionByIdOptions CopyTransactionOptions = { };
		CopyTransactionOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_COPYTRANSACTIONBYID_API_LATEST, 1);
		CopyTransactionOptions.LocalUserId = Data->LocalUserId;
		CopyTransactionOptions.TransactionId = Data->TransactionId;
		EOS_Ecom_HTransaction TransactionHandle;
		const EOS_EResult CopyTransactionResult = EOS_Ecom_CopyTransactionById(EOSSubsystem->EcomHandle, &CopyTransactionOptions, &TransactionHandle);
		if (CopyTransactionResult == EOS_EResult::EOS_Success)
		{
			EOS_Ecom_Transaction_GetEntitlementsCountOptions EntitlementCountOptions = { };
			EntitlementCountOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_TRANSACTION_GETENTITLEMENTSCOUNT_API_LATEST, 1);
			const uint32_t EntitlementsCount = EOS_Ecom_Transaction_GetEntitlementsCount(TransactionHandle, &EntitlementCountOptions);

			for (uint32_t i = 0; i < EntitlementsCount; i++)
			{
				EOS_Ecom_Transaction_CopyEntitlementByIndexOptions Options = { };
				Options.ApiVersion = 1;
				UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_TRANSACTION_COPYENTITLEMENTBYINDEX_API_LATEST, 1);
				Options.EntitlementIndex = i;
				EOS_Ecom_Entitlement* Entitlement;
				const EOS_EResult CopyEntitlementResult = EOS_Ecom_Transaction_CopyEntitlementByIndex(TransactionHandle, &Options, &Entitlement);

				// The Receipt structure is 1 EOS_Ecom_Entitlement (mapped to FReceiptOfferEntry) has 1 line item
				if (CopyEntitlementResult == EOS_EResult::EOS_Success)
				{
					// OfferId is set to an empty string as the OfferId is not retrievable in the checkout callback
					PurchaseReceipt->AddReceiptOffer(FOfferNamespace(), "", 1);
					FPurchaseReceipt::FLineItemInfo& LineItem = PurchaseReceipt->ReceiptOffers[i].LineItems.Emplace_GetRef();
					// The item that is owned is the EOS_Ecom_CatalogItemId
					LineItem.UniqueId = Entitlement->CatalogItemId ? StringCast<TCHAR>(Entitlement->CatalogItemId).Get() : FUniqueEntitlementId();
					// If the item can be redeemed, store the EntitlementId so it can be passed to FinalizeReceiptValidationInfo()
					FString EntitlementId = Entitlement->EntitlementId ? StringCast<TCHAR>(Entitlement->EntitlementId).Get() : FString();
					LineItem.ValidationInfo = Entitlement->bRedeemed ? FString() : EntitlementId;

					EOS_Ecom_CatalogItem* CatalogItem = nullptr;
					EOS_Ecom_CopyItemByIdOptions CopyCatalogItemOptions = {};
					CopyCatalogItemOptions.ApiVersion = 1;
					UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_COPYITEMBYID_API_LATEST, 1);
					CopyCatalogItemOptions.ItemId = Entitlement->CatalogItemId;
					CopyCatalogItemOptions.LocalUserId = Data->LocalUserId;
					EOS_EResult CopyCatalogItemResult = EOS_Ecom_CopyItemById(EOSSubsystem->EcomHandle, &CopyCatalogItemOptions, &CatalogItem);

					if (CopyCatalogItemResult == EOS_EResult::EOS_Success || CopyCatalogItemResult == EOS_EResult::EOS_Ecom_CatalogItemStale)
					{
						LineItem.ItemName = CatalogItem->TitleText ? StringCast<TCHAR>(CatalogItem->TitleText).Get() : FString();
						EOS_Ecom_CatalogItem_Release(CatalogItem);
					}
					else
					{
						// EOS_Ecom_CopyItemById will return EOS_NotFound is QueryOffers has NOT been called OR if the EOS_Ecom_CatalogItemId doesn't tie back to a specific offer (bundles, season packs, etc...)
						UE_LOG_ONLINE(Verbose, TEXT("[FOnlinePurchaseEOS::Checkout] EOS_Ecom_CopyItemById: failed with error (%s)"), *LexToString(CopyCatalogItemResult));
					}
					EOS_Ecom_Entitlement_Release(Entitlement);
				}
				else
				{
					UE_LOG_ONLINE(Error, TEXT("[FOnlinePurchaseEOS::Checkout] EOS_Ecom_Transaction_CopyEntitlementByIndex: failed with error (%s)"), *LexToString(CopyEntitlementResult));
					OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Unknown), MakeShared<FPurchaseReceipt>());
					EOS_Ecom_Transaction_Release(TransactionHandle);
					return;
				}
			}
			EOS_Ecom_Transaction_Release(TransactionHandle);
			OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Success), PurchaseReceipt);
		}
		else
		{
			OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Unknown), MakeShared<FPurchaseReceipt>());
		}
	};

	EOS_Ecom_Checkout(EOSSubsystem->EcomHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
}

void FOnlinePurchaseEOS::Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseReceiptlessCheckoutComplete& Delegate)
{
	const FUniqueNetIdEOS& UserEOSId = FUniqueNetIdEOS::Cast(UserId);
	const EOS_EpicAccountId AccountId = UserEOSId.GetEpicAccountId();
	if (AccountId == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("[%hs] EpicAccountId is null"), __FUNCTION__);
		Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::InvalidUser));
		return;
	}
	if (CheckoutRequest.PurchaseOffers.Num() == 0)
	{
		UE_LOG_ONLINE(Error, TEXT("[%hs] PurchaseOffers.Num() is 0"), __FUNCTION__);
		Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::InvalidParams));
		return;
	}
	if (CheckoutRequest.PurchaseOffers.Num() > EOS_ECOM_CHECKOUT_MAX_ENTRIES)
	{
		UE_LOG_ONLINE(Error, TEXT("[%hs] PurcahseOffer.Num() over maximum of %d "), __FUNCTION__, EOS_ECOM_CHECKOUT_MAX_ENTRIES);
		Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::InvalidParams));
		return;
	}

	const int32 NumItems = CheckoutRequest.PurchaseOffers.Num();
	TArray<EOS_Ecom_CheckoutEntry> Entries;
	TArray<FTCHARToUTF8> OfferIdsUtf8;
	Entries.AddZeroed(NumItems);
	OfferIdsUtf8.Reserve(NumItems);

	for (int32 Index = 0; Index < NumItems; Index++)
	{
		OfferIdsUtf8.Emplace(*CheckoutRequest.PurchaseOffers[Index].OfferId);
		Entries[Index].ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_CHECKOUTENTRY_API_LATEST, 1);
		Entries[Index].OfferId = OfferIdsUtf8.Last().Get();
	}

	EOS_Ecom_CheckoutOptions Options = { };
	Options.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_CHECKOUT_API_LATEST, 2);
	Options.LocalUserId = AccountId;
	Options.EntryCount = NumItems;
	Options.Entries = (const EOS_Ecom_CheckoutEntry*)Entries.GetData();
	Options.OverrideCatalogNamespace = nullptr;
	Options.PreferredOrientation = EOS_ECheckoutOrientation::EOS_ECO_Default;

	FPurchaseCheckoutCallback* CallbackObj = new FPurchaseCheckoutCallback(FOnlinePurchaseEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, OnComplete = FOnPurchaseReceiptlessCheckoutComplete(Delegate)](const EOS_Ecom_CheckoutCallbackInfo* Data)
	{
		const EOS_EResult CheckoutResult = Data->ResultCode;
		if (CheckoutResult != EOS_EResult::EOS_Success)
		{
			if (CheckoutResult == EOS_EResult::EOS_Canceled)
			{
				UE_LOG_ONLINE(Log, TEXT("[FOnlinePurchaseEOS::Checkout] EOS_Ecom_Checkout was canceled by user"));
				OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Canceled));
			}
			else if (CheckoutResult == EOS_EResult::EOS_Ecom_PurchaseProcessing)
			{
				// See header documentation for EOS_Ecom_PurchaseProcessing and online EOS documentation for more information
				UE_LOG_ONLINE(Log, TEXT("[FOnlinePurchaseEOS::Checkout] QueryEntitlements will need to be called to confirm the purchase. This can be done via a UI button"));
				OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::FailExtended, LexToString(CheckoutResult)));
			}
			else
			{
				UE_LOG_ONLINE(Error, TEXT("[FOnlinePurchaseEOS::Checkout] EOS_Ecom_Checkout: failed with error (%s)"), *LexToString(CheckoutResult));
				OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Unknown));
			}
			return;
		}
		else
		{
			OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Success));
		}
	};

	EOS_Ecom_Checkout(EOSSubsystem->EcomHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
}

void FOnlinePurchaseEOS::FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId)
{
	UE_LOG_ONLINE(Verbose, TEXT("[%hs] Not supported. Did you mean FinalizeReceiptValidationInfo?"), __FUNCTION__);
}

void FOnlinePurchaseEOS::RedeemCode(const FUniqueNetId& UserId, const FRedeemCodeRequest& RedeemCodeRequest, const FOnPurchaseRedeemCodeComplete& Delegate)
{
	UE_LOG_ONLINE(Verbose, TEXT("[%hs] Not supported"), __FUNCTION__);
	static const TSharedRef<FPurchaseReceipt> BlankReceipt(MakeShared<FPurchaseReceipt>());
	Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::NotImplemented), BlankReceipt);
}

typedef TEOSCallback<EOS_Ecom_OnQueryOwnershipBySandboxIdsCallback, EOS_Ecom_QueryOwnershipBySandboxIdsCallbackInfo, FOnlinePurchaseEOS> FPurchaseQueryOwnershipBySandboxIdsCallback;

void FOnlinePurchaseEOS::QueryReceipts(const FUniqueNetId& UserId, bool /* bRestoreReceipts */, const FOnQueryReceiptsComplete& Delegate)
{
	const FUniqueNetIdEOS& UserEOSId = FUniqueNetIdEOS::Cast(UserId);
	const EOS_EpicAccountId AccountId = UserEOSId.GetEpicAccountId();
	if (AccountId == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("[%hs] EpicAccountId is null"), __FUNCTION__);
		Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::InvalidUser));
		return;
	}

	CachedReceipts.Reset();

	// Retrieve the SandboxId from config
	FString SandboxIdString;
	const FEOSSettings& EOSSettings = UEOSSettings::GetSettings();
	if (EOSSettings.bUseNamedPlatformConfig)
	{
		const FEOSSDKPlatformConfig* const PlatformConfig = EOSSubsystem->EOSSDKManager->GetPlatformConfig(EOSSettings.PlatformConfigName);
		if (!PlatformConfig)
		{
			UE_LOG_ONLINE(Warning, TEXT("[%hs] GetPlatformConfig failed"), __FUNCTION__);
			Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Unknown));
			return;
		}
		SandboxIdString = PlatformConfig->SandboxId;
	}
	else
	{
		FEOSArtifactSettings ArtifactSettings;
		if (!UEOSSettings::GetSelectedArtifactSettings(ArtifactSettings))
		{
			UE_LOG_ONLINE(Warning, TEXT("[%hs] GetSelectedArtifactSettings failed"), __FUNCTION__);
			Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Unknown));
			return;
		}
		SandboxIdString = ArtifactSettings.SandboxId;
	}

	if (SandboxIdString.IsEmpty())
	{
		UE_LOG_ONLINE(Warning, TEXT("[%hs] Empty SandboxId"), __FUNCTION__);
		Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Unknown));
		return;
	}

	EOS_Ecom_QueryOwnershipBySandboxIdsOptions Options = {};
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_QUERYOWNERSHIPBYSANDBOXIDSOPTIONS_API_LATEST, 1);
	Options.LocalUserId = AccountId;
	Options.SandboxIdsCount = 1;
	const auto SandboxIdUtf8 = StringCast<UTF8CHAR>(*SandboxIdString);
	const char* SandboxId = (const char*)SandboxIdUtf8.Get();
	Options.SandboxIds = &SandboxId;

	FPurchaseQueryOwnershipBySandboxIdsCallback* CallbackObj = new FPurchaseQueryOwnershipBySandboxIdsCallback(FOnlinePurchaseEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, OnComplete = FOnQueryReceiptsComplete(Delegate)](const EOS_Ecom_QueryOwnershipBySandboxIdsCallbackInfo* Data)
	{
		const EOS_EResult Result = Data->ResultCode;
		if (Result == EOS_EResult::EOS_Success)
		{
			UE_LOG_ONLINE(VeryVerbose, TEXT("[FOnlinePurchaseEOS::QueryReceipts] Sandbox Id Ownership Count: %d"), Data->SandboxIdItemOwnershipsCount);

			for (const EOS_Ecom_SandboxIdItemOwnership& SandboxOwnershipEntry : TArrayView<const EOS_Ecom_SandboxIdItemOwnership>(Data->SandboxIdItemOwnerships, Data->SandboxIdItemOwnershipsCount))
			{
				UE_LOG_ONLINE(VeryVerbose, TEXT("[FOnlinePurchaseEOS::QueryReceipts] Sandbox Id: %hs"), SandboxOwnershipEntry.SandboxId);
				UE_LOG_ONLINE(VeryVerbose, TEXT("[FOnlinePurchaseEOS::QueryReceipts] Ownership Count: %d"), SandboxOwnershipEntry.OwnedCatalogItemIdsCount);

				for (uint32_t CatalogItemIndex = 0; CatalogItemIndex < SandboxOwnershipEntry.OwnedCatalogItemIdsCount; CatalogItemIndex++)
				{
					const EOS_Ecom_CatalogItemId& CatalogItemId = SandboxOwnershipEntry.OwnedCatalogItemIds[CatalogItemIndex];

					FPurchaseReceipt& PurchaseReceipt = CachedReceipts.Emplace_GetRef();
					PurchaseReceipt.TransactionState = EPurchaseTransactionState::Purchased;

					UE_LOG_ONLINE(VeryVerbose, TEXT("[FOnlinePurchaseEOS::QueryReceipts] Catalog Item %d"), CatalogItemIndex);
					UE_LOG_ONLINE(VeryVerbose, TEXT("[FOnlinePurchaseEOS::QueryReceipts] Catalog Item Id: %hs"), CatalogItemId);

					// The Receipt structure is 1 FReceiptOfferEntry with 1 line item for each EOS_Ecom_CatalogItem the player owns
					// Receipts do not contain any entitlements - use QueryEntitlements in the Online Entitlements interface to retrieve entitlement data
					
					// OfferId is set to an empty string as the OfferId is not retrievable via EOS SDK QueryOwnership APIs
					PurchaseReceipt.AddReceiptOffer(FOfferNamespace(), "", 1);
					FPurchaseReceipt::FLineItemInfo& LineItem = PurchaseReceipt.ReceiptOffers[0].LineItems.Emplace_GetRef();
					// The item that is owned is the CatalogItemId
					LineItem.UniqueId = StringCast<TCHAR>(CatalogItemId).Get();

					EOS_Ecom_CatalogItem* CatalogItem = nullptr;
					EOS_Ecom_CopyItemByIdOptions CopyCatalogItemOptions = {};
					CopyCatalogItemOptions.ApiVersion = 1;
					UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_COPYITEMBYID_API_LATEST, 1);
					CopyCatalogItemOptions.ItemId = CatalogItemId;
					CopyCatalogItemOptions.LocalUserId = Data->LocalUserId;
					EOS_EResult CopyCatalogItemResult = EOS_Ecom_CopyItemById(EOSSubsystem->EcomHandle, &CopyCatalogItemOptions, &CatalogItem);

					if (CopyCatalogItemResult == EOS_EResult::EOS_Success || CopyCatalogItemResult == EOS_EResult::EOS_Ecom_CatalogItemStale)
					{
						LineItem.ItemName = CatalogItem->TitleText ? StringCast<TCHAR>(CatalogItem->TitleText).Get() : FString();
						EOS_Ecom_CatalogItem_Release(CatalogItem);
					}
					else
					{
						// EOS_Ecom_CopyItemById will return EOS_NotFound is QueryOffers has NOT been called OR if the catalogItemId doesn't tie back to a specific offer (bundles, season packs, etc...)
						UE_LOG_ONLINE(Verbose, TEXT("[FOnlinePurchaseEOS::QueryReceipts] EOS_Ecom_CopyItemById: failed with error (%s)"), *LexToString(CopyCatalogItemResult));
					}
				}
			}
			OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Success));
		}
		else
		{
			UE_LOG_ONLINE(Error, TEXT("[FOnlinePurchaseEOS::QueryReceipts] EOS_Ecom_QueryOwnership: failed with error (%s)"), *LexToString(Data->ResultCode));
			OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Unknown));
			return;
		}
	};

	EOS_Ecom_QueryOwnershipBySandboxIds(EOSSubsystem->EcomHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
}

void FOnlinePurchaseEOS::GetReceipts(const FUniqueNetId& UserId, TArray<FPurchaseReceipt>& OutReceipts) const
{
	OutReceipts.Empty(); 

	const FUniqueNetIdEOS& UserEOSId = FUniqueNetIdEOS::Cast(UserId);
	const EOS_EpicAccountId AccountId = UserEOSId.GetEpicAccountId();
	if (AccountId == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("[%hs] EpicAccountId is null"), __FUNCTION__);
		return;
	}

	if (CachedReceipts.IsEmpty())
	{
		UE_LOG_ONLINE(Error, TEXT("[%hs] CachedReceipts empty, call QueryReceipts"), __FUNCTION__);
		return;
	}

	OutReceipts = CachedReceipts;
}

typedef TEOSCallback<EOS_Ecom_OnRedeemEntitlementsCallback, EOS_Ecom_RedeemEntitlementsCallbackInfo, FOnlinePurchaseEOS> FPurchaseRedeemReceiptCallback;

void FOnlinePurchaseEOS::FinalizeReceiptValidationInfo(const FUniqueNetId& UserId, FString& InReceiptValidationInfo, const FOnFinalizeReceiptValidationInfoComplete& Delegate)
{
	const FUniqueNetIdEOS& UserEOSId = FUniqueNetIdEOS::Cast(UserId);
	const EOS_EpicAccountId AccountId = UserEOSId.GetEpicAccountId();
	if (AccountId == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("[%hs] EpicAccountId is null"), __FUNCTION__);
		Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::InvalidUser), InReceiptValidationInfo);
		return;
	}
	if (InReceiptValidationInfo.IsEmpty())
	{
		UE_LOG_ONLINE(Error, TEXT("[%hs] InReceiptValidationInfo is empty"), __FUNCTION__);
		Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::InvalidParams), InReceiptValidationInfo);
		return;
	}

	char const* Ids[1];
	FTCHARToUTF8 EntitlementId(*InReceiptValidationInfo);
	Ids[0] = EntitlementId.Get();

	EOS_Ecom_RedeemEntitlementsOptions Options = { };
	Options.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_REDEEMENTITLEMENTS_API_LATEST, 2);
	Options.LocalUserId = AccountId;
	Options.EntitlementIdCount = 1;
	Options.EntitlementIds = Ids;

	FPurchaseRedeemReceiptCallback* CallbackObj = new FPurchaseRedeemReceiptCallback(FOnlinePurchaseEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, Info = FString(InReceiptValidationInfo), OnComplete = FOnFinalizeReceiptValidationInfoComplete(Delegate)](const EOS_Ecom_RedeemEntitlementsCallbackInfo* Data)
		{
			EOS_EResult Result = Data->ResultCode;
			if (Result == EOS_EResult::EOS_Success)
			{

				// Find the receipt in our list and mark as redeemed (clear the validation info) - It's possible the entitlement is NOT in our cached receipts
				for (FPurchaseReceipt& SearchReceipt : CachedReceipts)
				{
					if (!SearchReceipt.ReceiptOffers.IsEmpty())
					{
						if (!SearchReceipt.ReceiptOffers[0].LineItems.IsEmpty())
						{
							if (SearchReceipt.ReceiptOffers[0].LineItems[0].ValidationInfo == Info)
							{
								// Clearing this field tells the game it can't be redeemed - ConsumedCount on FOnlineEntitlement should be used to determine if an entitlement can be redeemed
								SearchReceipt.ReceiptOffers[0].LineItems[0].ValidationInfo.Empty();
								break;
							}
						}
					}
				}

				OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Success), Info);
			}
			else
			{
				UE_LOG_ONLINE(Error, TEXT("[FOnlinePurchaseEOS::FinalizeReceiptValidationInfo] EOS_Ecom_RedeemEntitlements: failed with error (%s)"), *LexToString(Data->ResultCode));
				OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Unknown), Info);
				return;
			}
		};
	EOS_Ecom_RedeemEntitlements(EOSSubsystem->EcomHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
}

bool FOnlinePurchaseEOS::HandlePurchaseExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (!EOSSubsystem || !EOSSubsystem->UserManager.IsValid() || !EOSSubsystem->PurchaseInterfacePtr.IsValid())
	{
		UE_LOG_ONLINE(Error, TEXT("[%hs] EOSSubsystem or UserManager or PurchaseInterface invalid"), __FUNCTION__);
		return false;
	}

	if (FParse::Command(&Cmd, TEXT("RECEIPTS")))
	{
		QueryReceipts(*EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(EOSSubsystem->UserManager->GetDefaultLocalUser()), false,
			FOnQueryReceiptsComplete::CreateLambda([this](const FOnlineError& Result)
				{
					if (Result.WasSuccessful())
					{
						UE_LOG_ONLINE(Log, TEXT("[FOnlinePurchaseEOS::HandlePurchaseExec] QueryReceipts success"));
						for (const FPurchaseReceipt& Receipt : CachedReceipts)
						{
							if (!Receipt.ReceiptOffers.IsEmpty())
							{
								if (!Receipt.ReceiptOffers[0].LineItems.IsEmpty())
								{
									UE_LOG_ONLINE(Log, TEXT("[FOnlinePurchaseEOS::HandlePurchaseExec] Player owns ItemId: %s"), *Receipt.ReceiptOffers[0].LineItems[0].UniqueId);
								}
							}
						}
					}
					else
					{
						UE_LOG_ONLINE(Error, TEXT("[FOnlinePurchaseEOS::HandlePurchaseExec] QueryReceipts error (%s)"), *Result.GetErrorRaw());
					}
				}));
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("FINALIZE"))) // ONLINE PURCHASES FINALIZE EntitlementId=<ENTITLEMENTID>
	{
		FString EntitlementId;
		FParse::Value(Cmd, TEXT("EntitlementId="), EntitlementId);

		FinalizeReceiptValidationInfo(*EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(EOSSubsystem->UserManager->GetDefaultLocalUser()), EntitlementId,
			FOnFinalizeReceiptValidationInfoComplete::CreateLambda([this](const FOnlineError& Result, const FString& Info)
				{
					if (Result.WasSuccessful())
					{
						UE_LOG_ONLINE(Log, TEXT("[FOnlinePurchaseEOS::HandlePurchaseExec] FinalizeReceiptValidationInfo success - info (%s)"), *Info);
					}
					else
					{
						UE_LOG_ONLINE(Error, TEXT("[FOnlinePurchaseEOS::HandlePurchaseExec] FinalizeReceiptValidationInfo error (%s)"), *Result.GetErrorRaw());
					}
				}));
		return true;
	}
	else
	{
		return false;
	}
}

#endif