// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Online/CoreOnline.h"
#include "Interfaces/OnlinePurchaseInterface.h"
#include "OnlineSubsystemEOSTypes.h"

#if WITH_EOS_SDK
#include "eos_ecom_types.h"

class UWorld;

/**
 * Checkout offers, retrieve receipts (owned EOS_Ecom_CatalogItemIds) and finalize receipt validation (redeem entitlement)
 */

class FOnlinePurchaseEOS :
	public IOnlinePurchase,
	public TSharedFromThis<FOnlinePurchaseEOS, ESPMode::ThreadSafe>
{
public:
	virtual ~FOnlinePurchaseEOS() = default;
	
	virtual bool IsAllowedToPurchase(const FUniqueNetId& UserId) override { return true; }
	virtual bool IsAllowedToPurchase(const FUniqueNetId& UserId, FOnlineError& Error) override
	{
		Error = FOnlineError(EOnlineErrorResult::Success);
		return true;
	}
	/** Checkout store offers and return a receipt. Receipt contains the EOS_Ecom_EntitlementId in the LineItem.ValidationInfo which can be redeemed via FinalizeReceiptValidationInfo */
	virtual void Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseCheckoutComplete& Delegate) override;
	/** Checkout store offers */
	virtual void Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseReceiptlessCheckoutComplete& Delegate) override;
	/** Not implemented */
	virtual void FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId) override;
	/** Not implemented */
	virtual void RedeemCode(const FUniqueNetId& UserId, const FRedeemCodeRequest& RedeemCodeRequest, const FOnPurchaseRedeemCodeComplete& Delegate) override;
	/* Retrieve and cache ownership data only - To retrieve entitlements use QueryEntitlements in the OnlineEntitlements interface */
	virtual void QueryReceipts(const FUniqueNetId& UserId, bool /* bRestoreReceipts */, const FOnQueryReceiptsComplete& Delegate) override;
	/* Retrieve cached ownership data */
	virtual void GetReceipts(const FUniqueNetId& UserId, TArray<FPurchaseReceipt>& OutReceipts) const override;
	/** Used to redeem an entitlement - Pass in the EntitlementId in InReceiptValidationInfo - The EntitlementId is provided in the CheckoutCallback with receipt in LineItem.ValidationInfo and is provided in GetAllEntitlements, GetEntitlement, and GetItemEntitlement in the OnlineEntitlements interface */
	virtual void FinalizeReceiptValidationInfo(const FUniqueNetId& UserId, FString& InReceiptValidationInfo, const FOnFinalizeReceiptValidationInfoComplete& Delegate) override;
	
	FOnlinePurchaseEOS(FOnlineSubsystemEOS* InSubsystem);
	
	bool HandlePurchaseExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);

private:
	/** Default constructor disabled */
	FOnlinePurchaseEOS() = delete;

	/** Reference to the main EOS subsystem */
	FOnlineSubsystemEOS* EOSSubsystem;

	/** List of receipts for the user */
	//@todo joeg - make this support multiple users
	TArray<FPurchaseReceipt> CachedReceipts;
};

typedef TSharedPtr<FOnlinePurchaseEOS, ESPMode::ThreadSafe> FOnlinePurchaseEOSPtr;
typedef TWeakPtr<FOnlinePurchaseEOS, ESPMode::ThreadSafe> FOnlinePurchaseEOSWeakPtr;

#endif
