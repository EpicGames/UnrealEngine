// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSessionEOS.h"

#if WITH_EOS_SDK

#include "Misc/CommandLine.h"
#include "Misc/Guid.h"
#include "Online/OnlineBase.h"
#include "Online/OnlineSessionNames.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemEOS.h"
#include "OnlineSubsystemEOSPrivate.h"
#include "OnlineSubsystemEOSTypes.h"
#include "UserManagerEOS.h"
#include "OnlineSubsystemUtils.h"
#include "OnlineAsyncTaskManager.h"
#include "SocketSubsystemEOS.h"
#include "NboSerializerEOS.h"
#include "InternetAddrEOS.h"
#include "IEOSSDKManager.h"
#include "NetDriverEOS.h"
#include "EOSVoiceChatUser.h"

#include "eos_sessions.h"
#include "eos_metrics.h"
#include "eos_lobby.h"

#define USES_PRESENCE_ATTRIBUTE_KEY FName("USESPRESENCE")

FString MakeStringFromAttributeValue(const EOS_Sessions_AttributeData* Attribute)
{
	switch (Attribute->ValueType)
	{
		case EOS_ESessionAttributeType::EOS_SAT_Int64:
		{
			int32 Value = Attribute->Value.AsInt64;
			return FString::Printf(TEXT("%d"), Value);
		}
		case EOS_ESessionAttributeType::EOS_SAT_Double:
		{
			double Value = Attribute->Value.AsDouble;
			return FString::Printf(TEXT("%f"), Value);
		}
		case EOS_ESessionAttributeType::EOS_SAT_String:
		{
			return FString(UTF8_TO_TCHAR(Attribute->Value.AsUtf8));
		}
	}
	return TEXT("");
}

bool IsSessionSettingTypeSupported(EOnlineKeyValuePairDataType::Type InType)
{
	switch (InType)
	{
		case EOnlineKeyValuePairDataType::Int32:
		case EOnlineKeyValuePairDataType::UInt32:
		case EOnlineKeyValuePairDataType::Int64:
		case EOnlineKeyValuePairDataType::Double:
		case EOnlineKeyValuePairDataType::String:
		case EOnlineKeyValuePairDataType::Float:
		case EOnlineKeyValuePairDataType::Bool:
		case EOnlineKeyValuePairDataType::Json:
		{
			return true;
		}
	}
	return false;
}

EOS_EOnlineComparisonOp ToEOSSearchOp(EOnlineComparisonOp::Type Op)
{
	switch (Op)
	{
		case EOnlineComparisonOp::Equals:
		{
			return EOS_EOnlineComparisonOp::EOS_OCO_EQUAL;
		}
		case EOnlineComparisonOp::NotEquals:
		{
			return EOS_EOnlineComparisonOp::EOS_OCO_NOTEQUAL;
		}
		case EOnlineComparisonOp::GreaterThan:
		{
			return EOS_EOnlineComparisonOp::EOS_OCO_GREATERTHAN;
		}
		case EOnlineComparisonOp::GreaterThanEquals:
		{
			return EOS_EOnlineComparisonOp::EOS_OCO_GREATERTHANOREQUAL;
		}
		case EOnlineComparisonOp::LessThan:
		{
			return EOS_EOnlineComparisonOp::EOS_OCO_LESSTHAN;
		}
		case EOnlineComparisonOp::LessThanEquals:
		{
			return EOS_EOnlineComparisonOp::EOS_OCO_LESSTHANOREQUAL;
		}
		case EOnlineComparisonOp::Near:
		{
			return EOS_EOnlineComparisonOp::EOS_OCO_DISTANCE;
		}
		case EOnlineComparisonOp::In:
		{
			return EOS_EOnlineComparisonOp::EOS_OCO_ANYOF;
		}
		case EOnlineComparisonOp::NotIn:
		{
			return EOS_EOnlineComparisonOp::EOS_OCO_NOTANYOF;
		}
	}
	return EOS_EOnlineComparisonOp::EOS_OCO_EQUAL;
}

struct FAttributeOptions :
	public EOS_Sessions_AttributeData
{
	char KeyAnsi[EOS_OSS_STRING_BUFFER_LENGTH];
	char ValueAnsi[EOS_OSS_STRING_BUFFER_LENGTH];

	FAttributeOptions(const char* InKey, const char* InValue) :
		EOS_Sessions_AttributeData()
	{
		ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_SESSIONATTRIBUTEDATA_API_LATEST, 1);
		ValueType = EOS_ESessionAttributeType::EOS_SAT_String;
		Value.AsUtf8 = ValueAnsi;
		Key = KeyAnsi;
		FCStringAnsi::Strncpy(KeyAnsi, InKey, EOS_OSS_STRING_BUFFER_LENGTH);
		FCStringAnsi::Strncpy(ValueAnsi, InValue, EOS_OSS_STRING_BUFFER_LENGTH);
	}

	FAttributeOptions(const char* InKey, bool InValue) :
		EOS_Sessions_AttributeData()
	{
		ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_SESSIONATTRIBUTEDATA_API_LATEST, 1);
		ValueType = EOS_ESessionAttributeType::EOS_SAT_Boolean;
		Value.AsBool = InValue ? EOS_TRUE : EOS_FALSE;
		Key = KeyAnsi;
		FCStringAnsi::Strncpy(KeyAnsi, InKey, EOS_OSS_STRING_BUFFER_LENGTH);
	}

	FAttributeOptions(const char* InKey, float InValue) :
		EOS_Sessions_AttributeData()
	{
		ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_SESSIONATTRIBUTEDATA_API_LATEST, 1);
		ValueType = EOS_ESessionAttributeType::EOS_SAT_Double;
		Value.AsDouble = InValue;
		Key = KeyAnsi;
		FCStringAnsi::Strncpy(KeyAnsi, InKey, EOS_OSS_STRING_BUFFER_LENGTH);
	}

	FAttributeOptions(const char* InKey, int32 InValue) :
		EOS_Sessions_AttributeData()
	{
		ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_SESSIONATTRIBUTEDATA_API_LATEST, 1);
		ValueType = EOS_ESessionAttributeType::EOS_SAT_Int64;
		Value.AsInt64 = InValue;
		Key = KeyAnsi;
		FCStringAnsi::Strncpy(KeyAnsi, InKey, EOS_OSS_STRING_BUFFER_LENGTH);
	}

	FAttributeOptions(const char* InKey, const FVariantData& InValue) :
		EOS_Sessions_AttributeData()
	{
		ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_SESSIONATTRIBUTEDATA_API_LATEST, 1);

		switch (InValue.GetType())
		{
			case EOnlineKeyValuePairDataType::Int32:
			{
				ValueType = EOS_ESessionAttributeType::EOS_SAT_Int64;
				int32 RawValue = 0;
				InValue.GetValue(RawValue);
				Value.AsInt64 = RawValue;
				break;
			}
			case EOnlineKeyValuePairDataType::UInt32:
			{
				ValueType = EOS_ESessionAttributeType::EOS_SAT_Int64;
				uint32 RawValue = 0;
				InValue.GetValue(RawValue);
				Value.AsInt64 = RawValue;
				break;
			}
			case EOnlineKeyValuePairDataType::Int64:
			{
				ValueType = EOS_ESessionAttributeType::EOS_SAT_Int64;
				int64 RawValue = 0;
				InValue.GetValue(RawValue);
				Value.AsInt64 = RawValue;
				break;
			}
			case EOnlineKeyValuePairDataType::Bool:
			{
				ValueType = EOS_ESessionAttributeType::EOS_SAT_Boolean;
				bool RawValue = false;
				InValue.GetValue(RawValue);
				Value.AsBool = RawValue ? EOS_TRUE : EOS_FALSE;
				break;
			}
			case EOnlineKeyValuePairDataType::Double:
			{
				ValueType = EOS_ESessionAttributeType::EOS_SAT_Double;
				double RawValue = 0.0;
				InValue.GetValue(RawValue);
				Value.AsDouble = RawValue;
				break;
			}
			case EOnlineKeyValuePairDataType::Float:
			{
				ValueType = EOS_ESessionAttributeType::EOS_SAT_Double;
				float RawValue = 0.0;
				InValue.GetValue(RawValue);
				Value.AsDouble = RawValue;
				break;
			}
			case EOnlineKeyValuePairDataType::String:
			{
				ValueType = EOS_ESessionAttributeType::EOS_SAT_String;
				Value.AsUtf8 = ValueAnsi;
				Key = KeyAnsi;
				FString OutString;
				InValue.GetValue(OutString);
				FCStringAnsi::Strncpy(ValueAnsi, TCHAR_TO_UTF8(*OutString), EOS_OSS_STRING_BUFFER_LENGTH);
				break;
			}
		}
		Key = KeyAnsi;
		FCStringAnsi::Strncpy(KeyAnsi, InKey, EOS_OSS_STRING_BUFFER_LENGTH);
	}
};

struct FLobbyAttributeOptions :
	public EOS_Lobby_AttributeData
{
	char KeyAnsi[EOS_OSS_STRING_BUFFER_LENGTH];
	char ValueAnsi[EOS_OSS_STRING_BUFFER_LENGTH];

	FLobbyAttributeOptions(const char* InKey, const char* InValue) :
		EOS_Lobby_AttributeData()
	{
		ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ATTRIBUTEDATA_API_LATEST, 1);
		ValueType = EOS_ELobbyAttributeType::EOS_SAT_String;
		Value.AsUtf8 = ValueAnsi;
		Key = KeyAnsi;
		FCStringAnsi::Strncpy(KeyAnsi, InKey, EOS_OSS_STRING_BUFFER_LENGTH);
		FCStringAnsi::Strncpy(ValueAnsi, InValue, EOS_OSS_STRING_BUFFER_LENGTH);
	}

	FLobbyAttributeOptions(const char* InKey, bool InValue) :
		EOS_Lobby_AttributeData()
	{
		ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ATTRIBUTEDATA_API_LATEST, 1);
		ValueType = EOS_ELobbyAttributeType::EOS_SAT_Boolean;
		Value.AsBool = InValue ? EOS_TRUE : EOS_FALSE;
		Key = KeyAnsi;
		FCStringAnsi::Strncpy(KeyAnsi, InKey, EOS_OSS_STRING_BUFFER_LENGTH);
	}

	FLobbyAttributeOptions(const char* InKey, float InValue) :
		EOS_Lobby_AttributeData()
	{
		ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ATTRIBUTEDATA_API_LATEST, 1);
		ValueType = EOS_ELobbyAttributeType::EOS_SAT_Double;
		Value.AsDouble = InValue;
		Key = KeyAnsi;
		FCStringAnsi::Strncpy(KeyAnsi, InKey, EOS_OSS_STRING_BUFFER_LENGTH);
	}

	FLobbyAttributeOptions(const char* InKey, int32 InValue) :
		EOS_Lobby_AttributeData()
	{
		ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ATTRIBUTEDATA_API_LATEST, 1);
		ValueType = EOS_ELobbyAttributeType::EOS_SAT_Int64;
		Value.AsInt64 = InValue;
		Key = KeyAnsi;
		FCStringAnsi::Strncpy(KeyAnsi, InKey, EOS_OSS_STRING_BUFFER_LENGTH);
	}

	FLobbyAttributeOptions(const char* InKey, const FVariantData& InValue) :
		EOS_Lobby_AttributeData()
	{
		ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ATTRIBUTEDATA_API_LATEST, 1);

		switch (InValue.GetType())
		{
		case EOnlineKeyValuePairDataType::Int32:
		{
			ValueType = EOS_ELobbyAttributeType::EOS_SAT_Int64;
			int32 RawValue = 0;
			InValue.GetValue(RawValue);
			Value.AsInt64 = RawValue;
			break;
		}
		case EOnlineKeyValuePairDataType::UInt32:
		{
			ValueType = EOS_ELobbyAttributeType::EOS_SAT_Int64;
			uint32 RawValue = 0;
			InValue.GetValue(RawValue);
			Value.AsInt64 = RawValue;
			break;
		}
		case EOnlineKeyValuePairDataType::Int64:
		{
			ValueType = EOS_ELobbyAttributeType::EOS_SAT_Int64;
			int64 RawValue = 0;
			InValue.GetValue(RawValue);
			Value.AsInt64 = RawValue;
			break;
		}
		case EOnlineKeyValuePairDataType::Bool:
		{
			ValueType = EOS_ELobbyAttributeType::EOS_SAT_Boolean;
			bool RawValue = false;
			InValue.GetValue(RawValue);
			Value.AsBool = RawValue ? EOS_TRUE : EOS_FALSE;
			break;
		}
		case EOnlineKeyValuePairDataType::Double:
		{
			ValueType = EOS_ELobbyAttributeType::EOS_SAT_Double;
			double RawValue = 0.0;
			InValue.GetValue(RawValue);
			Value.AsDouble = RawValue;
			break;
		}
		case EOnlineKeyValuePairDataType::Float:
		{
			ValueType = EOS_ELobbyAttributeType::EOS_SAT_Double;
			float RawValue = 0.0;
			InValue.GetValue(RawValue);
			Value.AsDouble = RawValue;
			break;
		}
		case EOnlineKeyValuePairDataType::String:
		case EOnlineKeyValuePairDataType::Json:
		{
			ValueType = EOS_ELobbyAttributeType::EOS_SAT_String;
			Value.AsUtf8 = ValueAnsi;
			Key = KeyAnsi;
			FString OutString;
			InValue.GetValue(OutString);
			FCStringAnsi::Strncpy(ValueAnsi, TCHAR_TO_UTF8(*OutString), EOS_OSS_STRING_BUFFER_LENGTH);
			break;
		}
		}
		Key = KeyAnsi;
		FCStringAnsi::Strncpy(KeyAnsi, InKey, EOS_OSS_STRING_BUFFER_LENGTH);
	}
};

FOnlineSessionInfoEOS FOnlineSessionInfoEOS::Create(FUniqueNetIdStringRef UniqueNetId)
{
	FOnlineSessionInfoEOS Result;
	Result.SessionId = UniqueNetId;
	return Result;
}

FOnlineSessionInfoEOS FOnlineSessionInfoEOS::Create(FUniqueNetIdStringRef UniqueNetId, const TSharedPtr<FSessionDetailsEOS>& SessionHandle)
{
	FOnlineSessionInfoEOS Result;
	Result.SessionId = UniqueNetId;
	Result.SessionHandle = SessionHandle;
	return Result;
}

FOnlineSessionInfoEOS FOnlineSessionInfoEOS::Create(FUniqueNetIdStringRef UniqueNetId, const TSharedPtr<FLobbyDetailsEOS>& LobbyHandle)
{
	FOnlineSessionInfoEOS Result;
	Result.SessionId = UniqueNetId;
	Result.LobbyHandle = LobbyHandle;
	return Result;
}

void FOnlineSessionInfoEOS::InitLAN(FOnlineSubsystemEOS* Subsystem)
{
	// Read the IP from the system
	bool bCanBindAll;
	HostAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalHostAddr(*GLog, bCanBindAll);

	// The below is a workaround for systems that set hostname to a distinct address from 127.0.0.1 on a loopback interface.
	// See e.g. https://www.debian.org/doc/manuals/debian-reference/ch05.en.html#_the_hostname_resolution
	// and http://serverfault.com/questions/363095/why-does-my-hostname-appear-with-the-address-127-0-1-1-rather-than-127-0-0-1-in
	// Since we bind to 0.0.0.0, we won't answer on 127.0.1.1, so we need to advertise ourselves as 127.0.0.1 for any other loopback address we may have.
	uint32 HostIp = 0;
	HostAddr->GetIp(HostIp); // will return in host order
	// if this address is on loopback interface, advertise it as 127.0.0.1
	if ((HostIp & 0xff000000) == 0x7f000000)
	{
		HostAddr->SetIp(0x7f000001);	// 127.0.0.1
	}

	// Now set the port that was configured
	HostAddr->SetPort(GetPortFromNetDriver(Subsystem->GetInstanceName()));

	FGuid OwnerGuid;
	FPlatformMisc::CreateGuid(OwnerGuid);
	SessionId = FUniqueNetIdEOSSession::Create(OwnerGuid.ToString());
}

typedef TEOSGlobalCallback<EOS_Sessions_OnSessionInviteReceivedCallback, EOS_Sessions_SessionInviteReceivedCallbackInfo, FOnlineSessionEOS> FSessionInviteReceivedCallback;
typedef TEOSGlobalCallback<EOS_Sessions_OnSessionInviteAcceptedCallback, EOS_Sessions_SessionInviteAcceptedCallbackInfo, FOnlineSessionEOS> FSessionInviteAcceptedCallback;
typedef TEOSGlobalCallback<EOS_Sessions_OnJoinSessionAcceptedCallback, EOS_Sessions_JoinSessionAcceptedCallbackInfo, FOnlineSessionEOS> FJoinSessionAcceptedCallback;
typedef TEOSGlobalCallback<EOS_Sessions_OnLeaveSessionRequestedCallback, EOS_Sessions_LeaveSessionRequestedCallbackInfo, FOnlineSessionEOS> FLeaveSessionRequestedCallback;

// Lobby session callbacks
typedef TEOSCallback<EOS_Lobby_OnCreateLobbyCallback, EOS_Lobby_CreateLobbyCallbackInfo, FOnlineSessionEOS> FLobbyCreatedCallback;
typedef TEOSCallback<EOS_Lobby_OnUpdateLobbyCallback, EOS_Lobby_UpdateLobbyCallbackInfo, FOnlineSessionEOS> FLobbyUpdatedCallback;
typedef TEOSCallback<EOS_Lobby_OnJoinLobbyCallback, EOS_Lobby_JoinLobbyCallbackInfo, FOnlineSessionEOS> FLobbyJoinedCallback;
typedef TEOSCallback<EOS_Lobby_OnLeaveLobbyCallback, EOS_Lobby_LeaveLobbyCallbackInfo, FOnlineSessionEOS> FLobbyLeftCallback;
typedef TEOSCallback<EOS_Lobby_OnDestroyLobbyCallback, EOS_Lobby_DestroyLobbyCallbackInfo, FOnlineSessionEOS> FLobbyDestroyedCallback;
typedef TEOSCallback<EOS_Lobby_OnSendInviteCallback, EOS_Lobby_SendInviteCallbackInfo, FOnlineSessionEOS> FLobbySendInviteCallback;
typedef TEOSCallback<EOS_Lobby_OnKickMemberCallback, EOS_Lobby_KickMemberCallbackInfo, FOnlineSessionEOS> FLobbyRemovePlayerCallback;
typedef TEOSCallback<EOS_LobbySearch_OnFindCallback, EOS_LobbySearch_FindCallbackInfo, FOnlineSessionEOS> FLobbySearchFindCallback;

// Lobby notification callbacks
typedef TEOSGlobalCallback<EOS_Lobby_OnLobbyUpdateReceivedCallback, EOS_Lobby_LobbyUpdateReceivedCallbackInfo, FOnlineSessionEOS> FLobbyUpdateReceivedCallback;
typedef TEOSGlobalCallback<EOS_Lobby_OnLobbyMemberUpdateReceivedCallback, EOS_Lobby_LobbyMemberUpdateReceivedCallbackInfo, FOnlineSessionEOS> FLobbyMemberUpdateReceivedCallback;
typedef TEOSGlobalCallback<EOS_Lobby_OnLobbyMemberStatusReceivedCallback, EOS_Lobby_LobbyMemberStatusReceivedCallbackInfo, FOnlineSessionEOS> FLobbyMemberStatusReceivedCallback;
typedef TEOSGlobalCallback<EOS_Lobby_OnLobbyInviteReceivedCallback, EOS_Lobby_LobbyInviteReceivedCallbackInfo, FOnlineSessionEOS> FLobbyInviteReceivedCallback;
typedef TEOSGlobalCallback<EOS_Lobby_OnLobbyInviteAcceptedCallback, EOS_Lobby_LobbyInviteAcceptedCallbackInfo, FOnlineSessionEOS> FLobbyInviteAcceptedCallback;
typedef TEOSGlobalCallback<EOS_Lobby_OnJoinLobbyAcceptedCallback, EOS_Lobby_JoinLobbyAcceptedCallbackInfo, FOnlineSessionEOS> FJoinLobbyAcceptedCallback;
typedef TEOSGlobalCallback<EOS_Lobby_OnLeaveLobbyRequestedCallback, EOS_Lobby_LeaveLobbyRequestedCallbackInfo, FOnlineSessionEOS> FLeaveLobbyRequestedCallback;

FOnlineSessionEOS::~FOnlineSessionEOS()
{
	EOS_Sessions_RemoveNotifySessionInviteReceived(EOSSubsystem->SessionsHandle, SessionInviteReceivedId);
	EOS_Sessions_RemoveNotifySessionInviteAccepted(EOSSubsystem->SessionsHandle, SessionInviteAcceptedId);
	EOS_Sessions_RemoveNotifyJoinSessionAccepted(EOSSubsystem->SessionsHandle, JoinSessionAcceptedId);
	EOS_Sessions_RemoveNotifyLeaveSessionRequested(EOSSubsystem->SessionsHandle, LeaveSessionRequestedId);
	delete SessionInviteReceivedCallback;
	delete SessionInviteAcceptedCallback;
	delete JoinSessionAcceptedCallback;
	delete LeaveSessionRequestedCallback;

	EOS_Lobby_RemoveNotifyLobbyUpdateReceived(LobbyHandle, LobbyUpdateReceivedId);
	EOS_Lobby_RemoveNotifyLobbyMemberUpdateReceived(LobbyHandle, LobbyMemberUpdateReceivedId);
	EOS_Lobby_RemoveNotifyLobbyMemberStatusReceived(LobbyHandle, LobbyMemberStatusReceivedId);
	EOS_Lobby_RemoveNotifyLobbyInviteReceived(LobbyHandle, LobbyInviteReceivedId);
	EOS_Lobby_RemoveNotifyLobbyInviteAccepted(LobbyHandle, LobbyInviteAcceptedId);
	EOS_Lobby_RemoveNotifyJoinLobbyAccepted(LobbyHandle, JoinLobbyAcceptedId);
	EOS_Lobby_RemoveNotifyLeaveLobbyRequested(LobbyHandle, LeaveLobbyRequestedId);

	delete LobbyUpdateReceivedCallback;
	delete LobbyMemberUpdateReceivedCallback;
	delete LobbyMemberStatusReceivedCallback;
	delete LobbyInviteReceivedCallback;
	delete LobbyInviteAcceptedCallback;
	delete JoinLobbyAcceptedCallback;
	delete LeaveLobbyRequestedCallback;
}

void FOnlineSessionEOS::Init()
{
	RegisterSessionNotifications();

	// Lobbies
	LobbyHandle = EOS_Platform_GetLobbyInterface(*EOSSubsystem->EOSPlatformHandle);
	RegisterLobbyNotifications();
}

/**
 * Searches the named session array for the specified session
 *
 * @param LobbyId the lobby id to search for
 *
 * @return pointer to the struct if found, NULL otherwise
 */
FNamedOnlineSession* FOnlineSessionEOS::GetNamedSessionFromLobbyId(const FUniqueNetIdEOSLobby& LobbyId)
{
	FNamedOnlineSession* Result = nullptr;

	FScopeLock ScopeLock(&SessionLock);
	for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
	{
		FNamedOnlineSession& Session = Sessions[SearchIndex];
		if (Session.SessionInfo.IsValid())
		{
			FOnlineSessionInfoEOS* SessionInfo = (FOnlineSessionInfoEOS*)Session.SessionInfo.Get();

			// We'll check if the session is a Lobby session before comparing the ids
			if (!Session.SessionSettings.bIsLANMatch && Session.SessionSettings.bUseLobbiesIfAvailable && *SessionInfo->SessionId == LobbyId)
			{
				Result = &Sessions[SearchIndex];
				break;
			}
		}
	}

	return Result;
}

/**
 * Searches the search results and invites arrays for the specified session
 *
 * @param LobbyId the lobby id to search for
 *
 * @return pointer to the struct if found, NULL otherwise
 */
FOnlineSessionSearchResult* FOnlineSessionEOS::GetSearchResultFromLobbyId(const FUniqueNetIdEOSLobby& LobbyId)
{
	FOnlineSessionSearchResult* Result = nullptr;

	TArray<FOnlineSessionSearchResult*> CombinedSearchResults;

	if (CurrentSessionSearch.IsValid())
	{
		for (FOnlineSessionSearchResult& SearchResult : CurrentSessionSearch->SearchResults)
		{
			CombinedSearchResults.Add(&SearchResult);
		}
	}

	if (LastInviteSearch.IsValid())
	{
		for (FOnlineSessionSearchResult& SearchResult : LastInviteSearch->SearchResults)
		{
			CombinedSearchResults.Add(&SearchResult);
		}
	}

	for (FOnlineSessionSearchResult* SearchResult : CombinedSearchResults)
	{
		const FOnlineSession& Session = SearchResult->Session;
		if (Session.SessionInfo.IsValid())
		{
			FOnlineSessionInfoEOS* SessionInfo = (FOnlineSessionInfoEOS*)Session.SessionInfo.Get();

			// We'll check if the session is a Lobby session before comparing the ids
			if (!Session.SessionSettings.bIsLANMatch && Session.SessionSettings.bUseLobbiesIfAvailable && *SessionInfo->SessionId == LobbyId)
			{
				Result = SearchResult;
				break;
			}
		}
	}

	return Result;
}

/**
 * Searches all local sessions containers for the specified session
 *
 * @param LobbyId the lobby id to search for
 *
 * @return pointer to the struct if found, NULL otherwise
 */
FOnlineSession* FOnlineSessionEOS::GetOnlineSessionFromLobbyId(const FUniqueNetIdEOSLobby& LobbyId)
{
	// First we try to retrieve a named session matching the given lobby id

	FOnlineSession* Result = GetNamedSessionFromLobbyId(LobbyId);

	if (!Result)
	{
		// If no named session were found with that lobby id, we look amongst the sessions in the latest search results
		if (FOnlineSessionSearchResult* SearchResult = GetSearchResultFromLobbyId(LobbyId))
		{
			Result = &SearchResult->Session;
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionEOS::GetOnlineSessionFromLobbyId] Session with LobbyId [%s] not found."), *LobbyId.ToString());
		}
	}

	return Result;
}

int32 FOnlineSessionEOS::GetDefaultLocalUserForLobby(const FUniqueNetIdString& SessionId)
{
	if (FOnlineSession * Session = GetOnlineSessionFromLobbyId(FUniqueNetIdEOSLobby::Cast(SessionId)))
	{
		for (const TPair<FUniqueNetIdRef, FSessionSettings>& MemberSettings : Session->SessionSettings.MemberSettings)
		{
			int32 LocalUserId = EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(*MemberSettings.Key);
			
			if (LocalUserId != INVALID_LOCAL_USER)
			{
				return LocalUserId;
			}
		}
	}

	return INVALID_LOCAL_USER;
}

void FOnlineSessionEOS::RegisterSessionNotifications()
{
	// Register for session invite received notifications
	FSessionInviteReceivedCallback* SessionInviteReceivedCallbackObj = new FSessionInviteReceivedCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	SessionInviteReceivedCallback = SessionInviteReceivedCallbackObj;
	SessionInviteReceivedCallbackObj->CallbackLambda = [this](const EOS_Sessions_SessionInviteReceivedCallbackInfo* Data)
	{
		OnSessionInviteReceived(Data);
	};
	EOS_Sessions_AddNotifySessionInviteReceivedOptions AddNotifySessionInviteReceivedOptions = { };
	AddNotifySessionInviteReceivedOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_ADDNOTIFYSESSIONINVITERECEIVED_API_LATEST, 1);
	SessionInviteReceivedId = EOS_Sessions_AddNotifySessionInviteReceived(EOSSubsystem->SessionsHandle, &AddNotifySessionInviteReceivedOptions, SessionInviteReceivedCallbackObj, SessionInviteReceivedCallbackObj->GetCallbackPtr());

	// Register for session invite accepted notifications
	FSessionInviteAcceptedCallback* SessionInviteAcceptedCallbackObj = new FSessionInviteAcceptedCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	SessionInviteAcceptedCallback = SessionInviteAcceptedCallbackObj;
	SessionInviteAcceptedCallbackObj->CallbackLambda = [this](const EOS_Sessions_SessionInviteAcceptedCallbackInfo* Data)
	{
		OnSessionInviteAccepted(Data);
	};
	EOS_Sessions_AddNotifySessionInviteAcceptedOptions AddNotifySessionInviteAcceptedOptions = { };
	AddNotifySessionInviteAcceptedOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_ADDNOTIFYSESSIONINVITEACCEPTED_API_LATEST, 1);
	SessionInviteAcceptedId = EOS_Sessions_AddNotifySessionInviteAccepted(EOSSubsystem->SessionsHandle, &AddNotifySessionInviteAcceptedOptions, SessionInviteAcceptedCallbackObj, SessionInviteAcceptedCallbackObj->GetCallbackPtr());

	// Register for join session accepted notifications
	FJoinSessionAcceptedCallback* JoinSessionAcceptedCallbackObj = new FJoinSessionAcceptedCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	JoinSessionAcceptedCallback = JoinSessionAcceptedCallbackObj;
	JoinSessionAcceptedCallbackObj->CallbackLambda = [this](const EOS_Sessions_JoinSessionAcceptedCallbackInfo* Data)
	{
		OnJoinSessionAccepted(Data);
	};
	EOS_Sessions_AddNotifyJoinSessionAcceptedOptions AddNotifyJoinSessionAcceptedOptions = { };
	AddNotifyJoinSessionAcceptedOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_ADDNOTIFYJOINSESSIONACCEPTED_API_LATEST, 1);
	JoinSessionAcceptedId = EOS_Sessions_AddNotifyJoinSessionAccepted(EOSSubsystem->SessionsHandle, &AddNotifyJoinSessionAcceptedOptions, JoinSessionAcceptedCallbackObj, JoinSessionAcceptedCallbackObj->GetCallbackPtr());

	// Requested session leave notifications
	EOS_Sessions_AddNotifyLeaveSessionRequestedOptions AddNotifyLeaveSessionRequestedOptions = { 0 };
	AddNotifyLeaveSessionRequestedOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_ADDNOTIFYLEAVESESSIONREQUESTED_API_LATEST, 1);

	FLeaveSessionRequestedCallback* LeaveSessionRequestedCallbackObj = new FLeaveSessionRequestedCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	LeaveSessionRequestedCallback = LeaveSessionRequestedCallbackObj;
	LeaveSessionRequestedCallbackObj->CallbackLambda = [this](const EOS_Sessions_LeaveSessionRequestedCallbackInfo* Data)
		{
			OnLeaveSessionRequested(Data);
		};

	LeaveSessionRequestedId = EOS_Sessions_AddNotifyLeaveSessionRequested(EOSSubsystem->SessionsHandle, &AddNotifyLeaveSessionRequestedOptions, LeaveSessionRequestedCallbackObj, LeaveSessionRequestedCallbackObj->GetCallbackPtr());
}


void FOnlineSessionEOS::OnSessionInviteReceived(const EOS_Sessions_SessionInviteReceivedCallbackInfo* Data)
{
	EOSSubsystem->UserManager->ResolveUniqueNetIds(EOSSubsystem->UserManager->GetDefaultLocalUser(), { Data->LocalUserId, Data->TargetUserId }, [this, LocalUserId = Data->LocalUserId, TargetUserId = Data->TargetUserId, InviteId = FString(UTF8_TO_TCHAR(Data->InviteId))](TMap<EOS_ProductUserId, FUniqueNetIdEOSRef> ResolvedUniqueNetIds, const FOnlineError& Error)
		{
			if (!ResolvedUniqueNetIds.Contains(LocalUserId))
			{
				// We'll print a warning but not trigger the delegate as we have no information to transmit with it
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot process invite due to unknown user (%s)"), *LexToString(LocalUserId));
				return;
			}

			if (!ResolvedUniqueNetIds.Contains(TargetUserId))
			{
				// We'll print a warning but not trigger the delegate as we have no information to transmit with it
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot process invite due to unknown user (%s)"), *LexToString(TargetUserId));
				return;
			}

			const FUniqueNetIdEOSRef NetId = ResolvedUniqueNetIds[LocalUserId];
			const FUniqueNetIdEOSRef FromNetId = ResolvedUniqueNetIds[TargetUserId];
			const int32 LocalUserNum = EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(*NetId);

			EOS_Sessions_CopySessionHandleByInviteIdOptions Options = { };
			Options.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_COPYSESSIONHANDLEBYINVITEID_API_LATEST, 1);
			const auto InviteIdUTF8 = StringCast<UTF8CHAR>(*InviteId);
			Options.InviteId = (const char*)InviteIdUTF8.Get();
			EOS_HSessionDetails SessionDetailsHandle = nullptr;
			EOS_EResult Result = EOS_Sessions_CopySessionHandleByInviteId(EOSSubsystem->SessionsHandle, &Options, &SessionDetailsHandle);
			if (Result == EOS_EResult::EOS_Success)
			{
				const TSharedRef<FSessionDetailsEOS> SessionDetails = MakeShared<FSessionDetailsEOS>(SessionDetailsHandle);
				LastInviteSearch = MakeShared<FOnlineSessionSearch>();
				AddSearchResult(SessionDetails, LastInviteSearch.ToSharedRef(), [this, NetId, FromNetId](bool bWasSuccessful)
					{
						TriggerOnSessionInviteReceivedDelegates(*NetId, *FromNetId, EOSSubsystem->GetAppId(), bWasSuccessful ? LastInviteSearch->SearchResults[0] : FOnlineSessionSearchResult());
					});
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("EOS_Sessions_CopySessionHandleByInviteId not successful. Finished with EOS_EResult %s"), *LexToString(Result));
				TriggerOnSessionInviteReceivedDelegates(*NetId, *FromNetId, EOSSubsystem->GetAppId(), FOnlineSessionSearchResult());
			}
		});
}

void FOnlineSessionEOS::OnSessionInviteAccepted(const EOS_Sessions_SessionInviteAcceptedCallbackInfo* Data)
{
	EOSSubsystem->UserManager->ResolveUniqueNetId(EOSSubsystem->UserManager->GetDefaultLocalUser(), Data->LocalUserId, [this, InviteId = FString(UTF8_TO_TCHAR(Data->InviteId))](FUniqueNetIdEOSRef ResolvedUniqueNetId, const FOnlineError& Error)
		{
			const int32 LocalUserNum = EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(*ResolvedUniqueNetId);

			EOS_Sessions_CopySessionHandleByInviteIdOptions Options = { };
			Options.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_COPYSESSIONHANDLEBYINVITEID_API_LATEST, 1);
			const auto InviteIdUTF8 = StringCast<UTF8CHAR>(*InviteId);
			Options.InviteId = (const char*)InviteIdUTF8.Get();
			EOS_HSessionDetails SessionDetailsHandle = nullptr;
			EOS_EResult Result = EOS_Sessions_CopySessionHandleByInviteId(EOSSubsystem->SessionsHandle, &Options, &SessionDetailsHandle);
			if (Result == EOS_EResult::EOS_Success)
			{
				const TSharedRef<FSessionDetailsEOS> SessionDetails = MakeShared<FSessionDetailsEOS>(SessionDetailsHandle);
				LastInviteSearch = MakeShared<FOnlineSessionSearch>();
				AddSearchResult(SessionDetails, LastInviteSearch.ToSharedRef(), [this, LocalUserNum, ResolvedUniqueNetId](bool bWasSuccessful)
					{
						TriggerOnSessionUserInviteAcceptedDelegates(true, LocalUserNum, ResolvedUniqueNetId, bWasSuccessful ? LastInviteSearch->SearchResults[0] : FOnlineSessionSearchResult());
					});
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("EOS_Sessions_CopySessionHandleByInviteId not successful. Finished with EOS_EResult %s"), *LexToString(Result));
				TriggerOnSessionUserInviteAcceptedDelegates(false, LocalUserNum, ResolvedUniqueNetId, FOnlineSessionSearchResult());
			}
		});
}

void FOnlineSessionEOS::OnJoinSessionAccepted(const EOS_Sessions_JoinSessionAcceptedCallbackInfo* Data)
{
	EOSSubsystem->UserManager->ResolveUniqueNetId(EOSSubsystem->UserManager->GetDefaultLocalUser(), Data->LocalUserId, [this, UiEventId = Data->UiEventId](FUniqueNetIdEOSRef ResolvedUniqueNetId, const FOnlineError& Error)
		{
			const int32 LocalUserNum = EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(*ResolvedUniqueNetId);

			EOS_Sessions_CopySessionHandleByUiEventIdOptions Options = { };
			Options.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_COPYSESSIONHANDLEBYUIEVENTID_API_LATEST, 1);
			Options.UiEventId = UiEventId;
			EOS_HSessionDetails SessionDetailsHandle = nullptr;
			EOS_EResult Result = EOS_Sessions_CopySessionHandleByUiEventId(EOSSubsystem->SessionsHandle, &Options, &SessionDetailsHandle);
			if (Result == EOS_EResult::EOS_Success)
			{
				const TSharedRef<FSessionDetailsEOS> SessionDetails = MakeShared<FSessionDetailsEOS>(SessionDetailsHandle);
				LastInviteSearch = MakeShared<FOnlineSessionSearch>();
				AddSearchResult(SessionDetails, LastInviteSearch.ToSharedRef(), [this, LocalUserNum, ResolvedUniqueNetId](bool bWasSuccessful)
					{
						TriggerOnSessionUserInviteAcceptedDelegates(bWasSuccessful, LocalUserNum, ResolvedUniqueNetId, bWasSuccessful ? LastInviteSearch->SearchResults[0] : FOnlineSessionSearchResult());
					});
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("EOS_Sessions_CopySessionHandleByUiEventId not successful. Finished with EOS_EResult %s"), *LexToString(Result));
				TriggerOnSessionUserInviteAcceptedDelegates(false, LocalUserNum, ResolvedUniqueNetId, FOnlineSessionSearchResult());
			}
		});
}

void FOnlineSessionEOS::OnLeaveSessionRequested(const EOS_Sessions_LeaveSessionRequestedCallbackInfo* Data)
{
	const int32 LocalUserNum = EOSSubsystem->UserManager->GetLocalUserNumFromProductUserId(Data->LocalUserId);
	if (LocalUserNum == INVALID_LOCAL_USER)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot process leave session request due to unknown local user (%s)"), *LexToString(Data->LocalUserId));
		return;
	}

	FName SessionName = UTF8_TO_TCHAR(Data->SessionName);
	const FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session == nullptr)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot process leave session request due to unknown session with name (%s)"), *SessionName.ToString());
		return;
	}

	TriggerOnDestroySessionRequestedDelegates(LocalUserNum, SessionName);
}

void FOnlineSessionEOS::RegisterLobbyNotifications()
{
	// Lobby data updates
	EOS_Lobby_AddNotifyLobbyUpdateReceivedOptions AddNotifyLobbyUpdateReceivedOptions = { 0 };
	AddNotifyLobbyUpdateReceivedOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ADDNOTIFYLOBBYUPDATERECEIVED_API_LATEST, 1);

	FLobbyUpdateReceivedCallback* LobbyUpdateReceivedCallbackObj = new FLobbyUpdateReceivedCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	LobbyUpdateReceivedCallback = LobbyUpdateReceivedCallbackObj;
	LobbyUpdateReceivedCallbackObj->CallbackLambda = [this](const EOS_Lobby_LobbyUpdateReceivedCallbackInfo* Data)
	{
		OnLobbyUpdateReceived(Data->LobbyId);
	};

	LobbyUpdateReceivedId = EOS_Lobby_AddNotifyLobbyUpdateReceived(LobbyHandle, &AddNotifyLobbyUpdateReceivedOptions, LobbyUpdateReceivedCallbackObj, LobbyUpdateReceivedCallbackObj->GetCallbackPtr());

	// Lobby member data updates
	EOS_Lobby_AddNotifyLobbyMemberUpdateReceivedOptions AddNotifyLobbyMemberUpdateReceivedOptions = { 0 };
	AddNotifyLobbyMemberUpdateReceivedOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ADDNOTIFYLOBBYMEMBERUPDATERECEIVED_API_LATEST, 1);

	FLobbyMemberUpdateReceivedCallback* LobbyMemberUpdateReceivedCallbackObj = new FLobbyMemberUpdateReceivedCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	LobbyMemberUpdateReceivedCallback = LobbyMemberUpdateReceivedCallbackObj;
	LobbyMemberUpdateReceivedCallbackObj->CallbackLambda = [this](const EOS_Lobby_LobbyMemberUpdateReceivedCallbackInfo* Data)
	{
		OnLobbyMemberUpdateReceived(Data->LobbyId, Data->TargetUserId);
	};

	LobbyMemberUpdateReceivedId = EOS_Lobby_AddNotifyLobbyMemberUpdateReceived(LobbyHandle, &AddNotifyLobbyMemberUpdateReceivedOptions, LobbyMemberUpdateReceivedCallbackObj, LobbyMemberUpdateReceivedCallbackObj->GetCallbackPtr());

	// Lobby member status updates (joined/left/disconnected/kicked/promoted)
	EOS_Lobby_AddNotifyLobbyMemberStatusReceivedOptions AddNotifyLobbyMemberStatusReceivedOptions = { 0 };
	AddNotifyLobbyMemberStatusReceivedOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ADDNOTIFYLOBBYMEMBERSTATUSRECEIVED_API_LATEST, 1);

	FLobbyMemberStatusReceivedCallback* LobbyMemberStatusReceivedCallbackObj = new FLobbyMemberStatusReceivedCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	LobbyMemberStatusReceivedCallback = LobbyMemberStatusReceivedCallbackObj;
	LobbyMemberStatusReceivedCallbackObj->CallbackLambda = [this](const EOS_Lobby_LobbyMemberStatusReceivedCallbackInfo* Data)
	{
		OnMemberStatusReceived(Data->LobbyId, Data->TargetUserId, Data->CurrentStatus);
	};

	LobbyMemberStatusReceivedId = EOS_Lobby_AddNotifyLobbyMemberStatusReceived(LobbyHandle, &AddNotifyLobbyMemberStatusReceivedOptions, LobbyMemberStatusReceivedCallbackObj, LobbyMemberStatusReceivedCallbackObj->GetCallbackPtr());

	// Received lobby invite notifications
	EOS_Lobby_AddNotifyLobbyInviteReceivedOptions AddNotifyLobbyInviteReceivedOptions = { 0 };
	AddNotifyLobbyInviteReceivedOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ADDNOTIFYLOBBYINVITERECEIVED_API_LATEST, 1);

	FLobbyInviteReceivedCallback* LobbyInviteReceivedCallbackObj = new FLobbyInviteReceivedCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	LobbyInviteReceivedCallback = LobbyInviteReceivedCallbackObj;
	LobbyInviteReceivedCallbackObj->CallbackLambda = [this](const EOS_Lobby_LobbyInviteReceivedCallbackInfo* Data)
	{
		OnLobbyInviteReceived(Data);
	};

	LobbyInviteReceivedId = EOS_Lobby_AddNotifyLobbyInviteReceived(LobbyHandle, &AddNotifyLobbyInviteReceivedOptions, LobbyInviteReceivedCallbackObj, LobbyInviteReceivedCallbackObj->GetCallbackPtr());

	// Accepted lobby invite notifications
	EOS_Lobby_AddNotifyLobbyInviteAcceptedOptions AddNotifyLobbyInviteAcceptedOptions = { 0 };
	AddNotifyLobbyInviteAcceptedOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ADDNOTIFYLOBBYINVITEACCEPTED_API_LATEST, 1);

	FLobbyInviteAcceptedCallback* LobbyInviteAcceptedCallbackObj = new FLobbyInviteAcceptedCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	LobbyInviteAcceptedCallback = LobbyInviteAcceptedCallbackObj;
	LobbyInviteAcceptedCallbackObj->CallbackLambda = [this](const EOS_Lobby_LobbyInviteAcceptedCallbackInfo* Data)
	{
		OnLobbyInviteAccepted(Data);
	};

	LobbyInviteAcceptedId = EOS_Lobby_AddNotifyLobbyInviteAccepted(LobbyHandle, &AddNotifyLobbyInviteAcceptedOptions, LobbyInviteAcceptedCallbackObj, LobbyInviteAcceptedCallbackObj->GetCallbackPtr());

	// Accepted lobby join notifications
	EOS_Lobby_AddNotifyJoinLobbyAcceptedOptions AddNotifyJoinLobbyAcceptedOptions = { 0 };
	AddNotifyJoinLobbyAcceptedOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ADDNOTIFYJOINLOBBYACCEPTED_API_LATEST, 1);

	FJoinLobbyAcceptedCallback* JoinLobbyAcceptedCallbackObj = new FJoinLobbyAcceptedCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	JoinLobbyAcceptedCallback = JoinLobbyAcceptedCallbackObj;
	JoinLobbyAcceptedCallbackObj->CallbackLambda = [this](const EOS_Lobby_JoinLobbyAcceptedCallbackInfo* Data)
	{
		OnJoinLobbyAccepted(Data);
	};

	JoinLobbyAcceptedId = EOS_Lobby_AddNotifyJoinLobbyAccepted(LobbyHandle, &AddNotifyJoinLobbyAcceptedOptions, JoinLobbyAcceptedCallbackObj, JoinLobbyAcceptedCallbackObj->GetCallbackPtr());

	// Requested lobby leave notifications
	EOS_Lobby_AddNotifyLeaveLobbyRequestedOptions AddNotifyLeaveLobbyRequestedOptions = { 0 };
	AddNotifyLeaveLobbyRequestedOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ADDNOTIFYLEAVELOBBYREQUESTED_API_LATEST, 1);

	FLeaveLobbyRequestedCallback* LeaveLobbyRequestedCallbackObj = new FLeaveLobbyRequestedCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	LeaveLobbyRequestedCallback = LeaveLobbyRequestedCallbackObj;
	LeaveLobbyRequestedCallbackObj->CallbackLambda = [this](const EOS_Lobby_LeaveLobbyRequestedCallbackInfo* Data)
	{
		OnLeaveLobbyRequested(Data);
	};

	LeaveLobbyRequestedId = EOS_Lobby_AddNotifyLeaveLobbyRequested(LobbyHandle, &AddNotifyLeaveLobbyRequestedOptions, LeaveLobbyRequestedCallbackObj, LeaveLobbyRequestedCallbackObj->GetCallbackPtr());
}

void FOnlineSessionEOS::OnLobbyUpdateReceived(const EOS_LobbyId& LobbyId)
{
	const FUniqueNetIdEOSLobbyRef LobbyNetId = FUniqueNetIdEOSLobby::Create(UTF8_TO_TCHAR(LobbyId));
	// Because the update might happen before lobby members have been populated, we'll use the default local user here to ensure its validity
	const EOS_ProductUserId LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(EOSSubsystem->UserManager->GetDefaultLocalUser());

	FNamedOnlineSession* Session = GetNamedSessionFromLobbyId(*LobbyNetId);
	if (Session)
	{
		EOS_Lobby_CopyLobbyDetailsHandleOptions Options = {};
		Options.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_COPYLOBBYDETAILSHANDLE_API_LATEST, 1);
		Options.LobbyId = LobbyId;
		Options.LocalUserId = LocalUserId;

		EOS_HLobbyDetails LobbyDetailsHandle;

		EOS_EResult CopyLobbyDetailsResult = EOS_Lobby_CopyLobbyDetailsHandle(LobbyHandle, &Options, &LobbyDetailsHandle);
		if (CopyLobbyDetailsResult == EOS_EResult::EOS_Success)
		{
			const TSharedRef<FLobbyDetailsEOS> LobbyDetails = MakeShared<FLobbyDetailsEOS>(LobbyDetailsHandle);

			EOS_LobbyDetails_Info* LobbyDetailsInfo = nullptr;
			EOS_LobbyDetails_CopyInfoOptions CopyOptions = { };
			CopyOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYDETAILS_COPYINFO_API_LATEST, 1);

			EOS_EResult CopyInfoResult = EOS_LobbyDetails_CopyInfo(LobbyDetails->LobbyDetailsHandle, &CopyOptions, &LobbyDetailsInfo);
			if (CopyInfoResult == EOS_EResult::EOS_Success)
			{
				// We are part of the lobby, so we'll be able to copy the member data
				CopyLobbyData(LobbyDetails, LobbyDetailsInfo, *Session, true, [this, SessionName = Session->SessionName](bool bWasSuccessful) {
					if (bWasSuccessful)
					{
						if (FNamedOnlineSession* Session = GetNamedSession(SessionName))
						{
							TriggerOnSessionSettingsUpdatedDelegates(SessionName, Session->SessionSettings);
						}
					}
				});
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::OnLobbyUpdateReceived] EOS_LobbyDetails_CopyInfo not successful. Finished with EOS_EResult %s"), *LexToString(CopyInfoResult));
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::OnLobbyUpdateReceived] EOS_Lobby_CopyLobbyDetailsHandle not successful. Finished with EOS_EResult %s"), *LexToString(CopyLobbyDetailsResult));
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::OnLobbyUpdateReceived] Unable to retrieve session with LobbyId %s"), *LobbyNetId->ToString());
	}
}

void FOnlineSessionEOS::OnLobbyMemberUpdateReceived(const EOS_LobbyId& LobbyId, const EOS_ProductUserId& TargetUserId)
{
	const FUniqueNetIdEOSLobbyRef LobbyNetId = FUniqueNetIdEOSLobby::Create(UTF8_TO_TCHAR(LobbyId));

	EOSSubsystem->UserManager->ResolveUniqueNetId(EOSSubsystem->UserManager->GetDefaultLocalUser(), TargetUserId, [this, TargetUserId, LobbyNetId](FUniqueNetIdEOSRef ResolvedUniqueNetId, const FOnlineError& Error)
		{
			UpdateOrAddLobbyMember(LobbyNetId, ResolvedUniqueNetId);
		});
}

void FOnlineSessionEOS::OnMemberStatusReceived(const EOS_LobbyId& LobbyId, const EOS_ProductUserId& TargetUserId, EOS_ELobbyMemberStatus CurrentStatus)
{
	const FUniqueNetIdEOSLobbyRef LobbyNetId = FUniqueNetIdEOSLobby::Create(UTF8_TO_TCHAR(LobbyId));
	FNamedOnlineSession* Session = GetNamedSessionFromLobbyId(*LobbyNetId);
	if (Session)
	{
		switch (CurrentStatus)
		{
		case EOS_ELobbyMemberStatus::EOS_LMS_JOINED:
			{
				EOSSubsystem->UserManager->ResolveUniqueNetId(EOSSubsystem->UserManager->GetDefaultLocalUser(), TargetUserId, [this, LobbyNetId](FUniqueNetIdEOSRef ResolvedUniqueNetId, const FOnlineError& Error)
					{
						UpdateOrAddLobbyMember(LobbyNetId, ResolvedUniqueNetId);
					});
			}

			break;
		case EOS_ELobbyMemberStatus::EOS_LMS_LEFT:
			{
				EOSSubsystem->UserManager->ResolveUniqueNetId(EOSSubsystem->UserManager->GetDefaultLocalUser(), TargetUserId, [this, LobbyNetId](FUniqueNetIdEOSRef ResolvedUniqueNetId, const FOnlineError& Error)
					{
						FNamedOnlineSession* Session = GetNamedSessionFromLobbyId(*LobbyNetId);
						if (Session)
						{
							RemoveOnlineSessionMember(Session->SessionName, ResolvedUniqueNetId);

							TriggerOnSessionParticipantLeftDelegates(Session->SessionName, *ResolvedUniqueNetId, EOnSessionParticipantLeftReason::Left);
						}
						else
						{
							UE_LOG_ONLINE_SESSION(VeryVerbose, TEXT("[FOnlineSessionEOS::OnMemberStatusReceived] EOS_LMS_LEFT: Unable to retrieve session with LobbyId %s"), *LobbyNetId->ToString());
						}
					});
			}
			break;
		case EOS_ELobbyMemberStatus::EOS_LMS_DISCONNECTED:
			{
				EOSSubsystem->UserManager->ResolveUniqueNetId(EOSSubsystem->UserManager->GetDefaultLocalUser(), TargetUserId, [this, LobbyNetId](FUniqueNetIdEOSRef ResolvedUniqueNetId, const FOnlineError& Error)
					{
						FNamedOnlineSession* Session = GetNamedSessionFromLobbyId(*LobbyNetId);
						if (Session)
						{
							RemoveOnlineSessionMember(Session->SessionName, ResolvedUniqueNetId);

							TriggerOnSessionParticipantLeftDelegates(Session->SessionName, *ResolvedUniqueNetId, EOnSessionParticipantLeftReason::Disconnected);
						}
						else
						{
							UE_LOG_ONLINE_SESSION(VeryVerbose, TEXT("[FOnlineSessionEOS::OnMemberStatusReceived] EOS_LMS_DISCONNECTED: Unable to retrieve session with LobbyId %s"), *LobbyNetId->ToString());
						}
					});
			}
			break;
		case EOS_ELobbyMemberStatus::EOS_LMS_KICKED:
			{
				EOSSubsystem->UserManager->ResolveUniqueNetId(EOSSubsystem->UserManager->GetDefaultLocalUser(), TargetUserId, [this, LobbyNetId](FUniqueNetIdEOSRef ResolvedUniqueNetId, const FOnlineError& Error)
					{
						FNamedOnlineSession* Session = GetNamedSessionFromLobbyId(*LobbyNetId);
						if (Session)
						{
							RemoveOnlineSessionMember(Session->SessionName, ResolvedUniqueNetId);

							TriggerOnSessionParticipantLeftDelegates(Session->SessionName, *ResolvedUniqueNetId, EOnSessionParticipantLeftReason::Kicked);
						}
						else
						{
							UE_LOG_ONLINE_SESSION(VeryVerbose, TEXT("[FOnlineSessionEOS::OnMemberStatusReceived] EOS_LMS_KICKED: Unable to retrieve session with LobbyId %s"), *LobbyNetId->ToString());
						}
					});
			}
			break;
		case EOS_ELobbyMemberStatus::EOS_LMS_PROMOTED:
			{
				EOSSubsystem->UserManager->ResolveUniqueNetId(EOSSubsystem->UserManager->GetDefaultLocalUser(), TargetUserId, [this, LobbyNetId](FUniqueNetIdEOSRef ResolvedUniqueNetId, const FOnlineError& Error)
					{
						FNamedOnlineSession* Session = GetNamedSessionFromLobbyId(*LobbyNetId);
						if (Session)
						{
							int32 DefaultLocalUser = EOSSubsystem->UserManager->GetDefaultLocalUser();
							FUniqueNetIdPtr LocalPlayerUniqueNetId = EOSSubsystem->UserManager->GetUniquePlayerId(DefaultLocalUser);

							if (*LocalPlayerUniqueNetId == *ResolvedUniqueNetId)
							{
								Session->OwningUserId = LocalPlayerUniqueNetId;
								Session->OwningUserName = EOSSubsystem->UserManager->GetPlayerNickname(DefaultLocalUser);;
								Session->bHosting = true;

								UpdateLobbySession(Session,
									FOnUpdateSessionCompleteDelegate::CreateLambda([this](FName SessionName, bool bWasSuccessful)
										{
											TriggerOnUpdateSessionCompleteDelegates(SessionName, bWasSuccessful);
										}));
							}

							// If we are not the new owner, the new owner will update the session and we'll receive the notification, updating ours as well
						}
						else
						{
							UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::OnMemberStatusReceived] EOS_LMS_PROMOTED: Unable to retrieve session with LobbyId %s"), *LobbyNetId->ToString());
						}
					});
			}
			break;
		case EOS_ELobbyMemberStatus::EOS_LMS_CLOSED:
			{
				const int32 DefaultLocalUser = EOSSubsystem->UserManager->GetDefaultLocalUser();
				DestroyLobbySession(DefaultLocalUser, Session, FOnDestroySessionCompleteDelegate::CreateLambda([](FName SessionName, bool bWasSuccessful) {}));
			}
			break;
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::OnMemberStatusReceived] Unable to retrieve session with LobbyId %s"), *LobbyNetId->ToString());
	}
}

void FOnlineSessionEOS::OnLobbyInviteReceived(const EOS_Lobby_LobbyInviteReceivedCallbackInfo* Data)
{
	const TArray<EOS_ProductUserId> PUIdS = { Data->LocalUserId, Data->TargetUserId };
	EOSSubsystem->UserManager->ResolveUniqueNetIds(EOSSubsystem->UserManager->GetDefaultLocalUser(), PUIdS, [this, LocalUserId = Data->LocalUserId, TargetUserId = Data->TargetUserId, InviteId = FString(UTF8_TO_TCHAR(Data->InviteId))](TMap<EOS_ProductUserId, FUniqueNetIdEOSRef> ResolvedUniqueNetIds, const FOnlineError& Error)
		{
			if (!ResolvedUniqueNetIds.Contains(LocalUserId))
			{
				// We'll print a warning but not trigger the delegate as we have no information to transmit with it
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot process invite due to unknown user (%s)"), *LexToString(LocalUserId));
				return;
			}

			if (!ResolvedUniqueNetIds.Contains(TargetUserId))
			{
				// We'll print a warning but not trigger the delegate as we have no information to transmit with it
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot process invite due to unknown user (%s)"), *LexToString(TargetUserId));
				return;
			}

			const FUniqueNetIdEOSRef NetId = ResolvedUniqueNetIds[LocalUserId];
			const FUniqueNetIdEOSRef FromNetId = ResolvedUniqueNetIds[TargetUserId];
			const int32 LocalUserNum = EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(*NetId);

			EOS_Lobby_CopyLobbyDetailsHandleByInviteIdOptions Options = { };
			Options.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_COPYLOBBYDETAILSHANDLEBYINVITEID_API_LATEST, 1);
			const auto InviteIdUTF8 = StringCast<UTF8CHAR>(*InviteId);
			Options.InviteId = (const char*)InviteIdUTF8.Get();
			EOS_HLobbyDetails LobbyDetailsHandle = nullptr;
			EOS_EResult Result = EOS_Lobby_CopyLobbyDetailsHandleByInviteId(LobbyHandle, &Options, &LobbyDetailsHandle);
			if (Result == EOS_EResult::EOS_Success)
			{
				const TSharedRef<FLobbyDetailsEOS> LobbyDetails = MakeShared<FLobbyDetailsEOS>(LobbyDetailsHandle);

				LastInviteSearch = MakeShared<FOnlineSessionSearch>();
				AddLobbySearchResult(LobbyDetails, LastInviteSearch.ToSharedRef(), [this, LocalUserNum, NetId, FromNetId](bool bWasSuccessful)
					{
						TriggerOnSessionInviteReceivedDelegates(*NetId, *FromNetId, EOSSubsystem->GetAppId(), bWasSuccessful ? LastInviteSearch->SearchResults.Last() : FOnlineSessionSearchResult());
					});
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("EOS_Lobby_CopyLobbyDetailsHandleByInviteId not successful. Finished with EOS_EResult %s"), *LexToString(Result));
				TriggerOnSessionInviteReceivedDelegates(*NetId, *FromNetId, EOSSubsystem->GetAppId(), FOnlineSessionSearchResult());
			}
		});
}

void FOnlineSessionEOS::OnLobbyInviteAccepted(const EOS_Lobby_LobbyInviteAcceptedCallbackInfo* Data)
{
	EOSSubsystem->UserManager->ResolveUniqueNetId(EOSSubsystem->UserManager->GetDefaultLocalUser(), Data->LocalUserId, [this, InviteId = FString(UTF8_TO_TCHAR(Data->InviteId))](FUniqueNetIdEOSRef ResolvedUniqueNetId, const FOnlineError& Error)
		{
			const int32 LocalUserNum = EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(*ResolvedUniqueNetId);

			EOS_Lobby_CopyLobbyDetailsHandleByInviteIdOptions Options = { };
			Options.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_COPYLOBBYDETAILSHANDLEBYINVITEID_API_LATEST, 1);
			const auto InviteIdUTF8 = StringCast<UTF8CHAR>(*InviteId);
			Options.InviteId = (const char*)InviteIdUTF8.Get();

			EOS_HLobbyDetails LobbyDetailsHandle;

			EOS_EResult Result = EOS_Lobby_CopyLobbyDetailsHandleByInviteId(LobbyHandle, &Options, &LobbyDetailsHandle);
			if (Result == EOS_EResult::EOS_Success)
			{
				const TSharedRef<FLobbyDetailsEOS> LobbyDetails = MakeShared<FLobbyDetailsEOS>(LobbyDetailsHandle);

				LastInviteSearch = MakeShared<FOnlineSessionSearch>();
				AddLobbySearchResult(LobbyDetails, LastInviteSearch.ToSharedRef(), [this, LocalUserNum, ResolvedUniqueNetId](bool bWasSuccessful)
					{
						// If we fail to copy the lobby data, we won't add a new search result, so we'll return an empty one
						TriggerOnSessionUserInviteAcceptedDelegates(bWasSuccessful, LocalUserNum, ResolvedUniqueNetId, bWasSuccessful ? LastInviteSearch->SearchResults.Last() : FOnlineSessionSearchResult());
					});
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::OnLobbyInviteAccepted] EOS_Lobby_CopyLobbyDetailsHandleByInviteId failed with EOS result code (%s)"), *LexToString(Result));
				TriggerOnSessionUserInviteAcceptedDelegates(false, LocalUserNum, ResolvedUniqueNetId, FOnlineSessionSearchResult());
			}
		});
}

void FOnlineSessionEOS::OnJoinLobbyAccepted(const EOS_Lobby_JoinLobbyAcceptedCallbackInfo* Data)
{
	EOSSubsystem->UserManager->ResolveUniqueNetId(EOSSubsystem->UserManager->GetDefaultLocalUser(), Data->LocalUserId, [this, UiEventId = Data->UiEventId](FUniqueNetIdEOSRef ResolvedUniqueNetId, const FOnlineError& Error)
		{
			const int32 LocalUserNum = EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(*ResolvedUniqueNetId);

			EOS_Lobby_CopyLobbyDetailsHandleByUiEventIdOptions Options = { 0 };
			Options.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_COPYLOBBYDETAILSHANDLEBYUIEVENTID_API_LATEST, 1);
			Options.UiEventId = UiEventId;

			EOS_HLobbyDetails LobbyDetailsHandle;
			EOS_EResult Result = EOS_Lobby_CopyLobbyDetailsHandleByUiEventId(LobbyHandle, &Options, &LobbyDetailsHandle);
			if (Result == EOS_EResult::EOS_Success)
			{
				const TSharedRef<FLobbyDetailsEOS> LobbyDetails = MakeShared<FLobbyDetailsEOS>(LobbyDetailsHandle);

				LastInviteSearch = MakeShared<FOnlineSessionSearch>();
				AddLobbySearchResult(LobbyDetails, LastInviteSearch.ToSharedRef(), [this, LocalUserNum, ResolvedUniqueNetId](bool bWasSuccessful)
					{
						// If we fail to copy the lobby data, we won't add a new search result, so we'll return an empty one
						TriggerOnSessionUserInviteAcceptedDelegates(bWasSuccessful, LocalUserNum, ResolvedUniqueNetId, bWasSuccessful ? LastInviteSearch->SearchResults.Last() : FOnlineSessionSearchResult());
					});
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::OnJoinLobbyAccepted] EOS_Lobby_CopyLobbyDetailsHandleByUiEventId failed with EOS result code (%s)"), *LexToString(Result));
				TriggerOnSessionUserInviteAcceptedDelegates(false, LocalUserNum, ResolvedUniqueNetId, FOnlineSessionSearchResult());
			}
		});
}

void FOnlineSessionEOS::OnLeaveLobbyRequested(const EOS_Lobby_LeaveLobbyRequestedCallbackInfo* Data)
{
	const int32 LocalUserNum = EOSSubsystem->UserManager->GetLocalUserNumFromProductUserId(Data->LocalUserId);
	if (LocalUserNum == INVALID_LOCAL_USER)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot process leave lobby request due to unknown local user (%s)"), *LexToString(Data->LocalUserId));
		return;
	}

	FString LobbyIdStr = UTF8_TO_TCHAR(Data->LobbyId);
	const FUniqueNetIdEOSLobbyRef LobbyNetId = FUniqueNetIdEOSLobby::Create(*LobbyIdStr);
	const FNamedOnlineSession* Session = GetNamedSessionFromLobbyId(*LobbyNetId);
	if (Session == nullptr)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot process leave lobby request due to unknown lobby with id (%s)"), *LobbyIdStr);
		return;
	}

	TriggerOnDestroySessionRequestedDelegates(LocalUserNum, Session->SessionName);
}

bool FOnlineSessionEOS::CreateSession(int32 HostingPlayerNum, FName SessionName, const FOnlineSessionSettings& NewSessionSettings)
{
	uint32 Result = ONLINE_FAIL;

	// Check for an existing session
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session == nullptr)
	{
		if (IsRunningDedicatedServer() || EOSSubsystem->UserManager->GetLoginStatus(HostingPlayerNum) >= ELoginStatus::UsingLocalProfile)
		{
			// Create a new session and deep copy the game settings
			Session = AddNamedSession(SessionName, NewSessionSettings);
			check(Session);
			Session->SessionState = EOnlineSessionState::Creating;

			Session->OwningUserId = EOSSubsystem->UserManager->GetUniquePlayerId(HostingPlayerNum);
			Session->OwningUserName = EOSSubsystem->UserManager->GetPlayerNickname(HostingPlayerNum);

			if (IsRunningDedicatedServer() || (Session->OwningUserId.IsValid() && Session->OwningUserId->IsValid()))
			{
				// RegisterPlayer will update these values for the local player
				Session->NumOpenPrivateConnections = NewSessionSettings.NumPrivateConnections;
				Session->NumOpenPublicConnections = NewSessionSettings.NumPublicConnections;

				Session->HostingPlayerNum = HostingPlayerNum;

				// Unique identifier of this build for compatibility
				Session->SessionSettings.BuildUniqueId = GetBuildUniqueId();

				// Create Internet or LAN match
				if (!NewSessionSettings.bIsLANMatch)
				{
					if (Session->SessionSettings.bUseLobbiesIfAvailable)
					{
						Result = CreateLobbySession(HostingPlayerNum, Session);
					}
					else
					{
						Result = CreateEOSSession(HostingPlayerNum, Session);
					}
				}
				else
				{
					Result = CreateLANSession(HostingPlayerNum, Session);
				}
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot create session '%s': invalid user (%d)."), *SessionName.ToString(), HostingPlayerNum);
			}

			if (Result != ONLINE_IO_PENDING)
			{
				// Set the game state as pending (not started)
				Session->SessionState = EOnlineSessionState::Pending;

				if (Result != ONLINE_SUCCESS)
				{
					// Clean up the session info so we don't get into a confused state
					RemoveNamedSession(SessionName);
				}
				else
				{
					RegisterLocalPlayers(Session);
				}
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot create session '%s': user not logged in (%d)."), *SessionName.ToString(), HostingPlayerNum);
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot create session '%s': session already exists."), *SessionName.ToString());
	}

	if (Result != ONLINE_IO_PENDING)
	{
		EOSSubsystem->ExecuteNextTick([this, SessionName, Result]()
			{
				TriggerOnCreateSessionCompleteDelegates(SessionName, Result == ONLINE_SUCCESS);
			});
	}

	return true;
}

bool FOnlineSessionEOS::CreateSession(const FUniqueNetId& HostingPlayerId, FName SessionName, const FOnlineSessionSettings& NewSessionSettings)
{
	return CreateSession(EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(HostingPlayerId), SessionName, NewSessionSettings);
}

uint32 FOnlineSessionEOS::CreateLANSession(int32 HostingPlayerNum, FNamedOnlineSession* Session)
{
	check(Session);
	uint32 Result = ONLINE_SUCCESS;

	// Setup the host session info
	FOnlineSessionInfoEOS* NewSessionInfo = new FOnlineSessionInfoEOS();
	NewSessionInfo->InitLAN(EOSSubsystem);
	Session->SessionInfo = MakeShareable(NewSessionInfo);

	// Don't create a the beacon if advertising is off
	if (Session->SessionSettings.bShouldAdvertise)
	{
		if (!LANSession.IsValid())
		{
			LANSession = MakeShareable(new FLANSession());
		}

		FOnValidQueryPacketDelegate QueryPacketDelegate = FOnValidQueryPacketDelegate::CreateRaw(this, &FOnlineSessionEOS::OnValidQueryPacketReceived);
		if (!LANSession->Host(QueryPacketDelegate))
		{
			Result = ONLINE_FAIL;
		}
	}

	return Result;
}

void FOnlineSessionEOS::SetPermissionLevel(EOS_HSessionModification SessionModHandle, FNamedOnlineSession* Session)
{
	EOS_SessionModification_SetPermissionLevelOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONMODIFICATION_SETPERMISSIONLEVEL_API_LATEST, 1);
	if (Session->SessionSettings.NumPublicConnections > 0)
	{
		Options.PermissionLevel = EOS_EOnlineSessionPermissionLevel::EOS_OSPF_PublicAdvertised;
	}
	else if (Session->SessionSettings.bAllowJoinViaPresence)
	{
		Options.PermissionLevel = EOS_EOnlineSessionPermissionLevel::EOS_OSPF_JoinViaPresence;
	}
	else
	{
		Options.PermissionLevel = EOS_EOnlineSessionPermissionLevel::EOS_OSPF_InviteOnly;
	}

	UE_LOG_ONLINE_SESSION(Log, TEXT("EOS_SessionModification_SetPermissionLevel() set to (%d) for session (%s)"), (int32)Options.PermissionLevel, *Session->SessionName.ToString());

	EOS_EResult ResultCode = EOS_SessionModification_SetPermissionLevel(SessionModHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("EOS_SessionModification_SetPermissionLevel() failed with EOS result code (%s)"), *LexToString(ResultCode));
	}
}

void FOnlineSessionEOS::SetMaxPlayers(EOS_HSessionModification SessionModHandle, FNamedOnlineSession* Session)
{
	EOS_SessionModification_SetMaxPlayersOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONMODIFICATION_SETMAXPLAYERS_API_LATEST, 1);
	Options.MaxPlayers = Session->SessionSettings.NumPrivateConnections + Session->SessionSettings.NumPublicConnections;

	UE_LOG_ONLINE_SESSION(Log, TEXT("EOS_SessionModification_SetMaxPlayers() set to (%d) for session (%s)"), Options.MaxPlayers, *Session->SessionName.ToString());

	const EOS_EResult ResultCode = EOS_SessionModification_SetMaxPlayers(SessionModHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("EOS_SessionModification_SetMaxPlayers() failed with EOS result code (%s)"), *LexToString(ResultCode));
	}
}

void FOnlineSessionEOS::SetInvitesAllowed(EOS_HSessionModification SessionModHandle, FNamedOnlineSession* Session)
{
	EOS_SessionModification_SetInvitesAllowedOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONMODIFICATION_SETINVITESALLOWED_API_LATEST, 1);
	Options.bInvitesAllowed = Session->SessionSettings.bAllowInvites ? EOS_TRUE : EOS_FALSE;

	UE_LOG_ONLINE_SESSION(Log, TEXT("EOS_SessionModification_SetInvitesAllowed() set to (%s) for session (%s)"), *LexToString(Options.bInvitesAllowed), *Session->SessionName.ToString());

	const EOS_EResult ResultCode = EOS_SessionModification_SetInvitesAllowed(SessionModHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("EOS_SessionModification_SetInvitesAllowed() failed with EOS result code (%s)"), *LexToString(ResultCode));
	}
}

void FOnlineSessionEOS::SetJoinInProgress(EOS_HSessionModification SessionModHandle, FNamedOnlineSession* Session)
{
	EOS_SessionModification_SetJoinInProgressAllowedOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONMODIFICATION_SETJOININPROGRESSALLOWED_API_LATEST, 1);
	Options.bAllowJoinInProgress = Session->SessionSettings.bAllowJoinInProgress ? EOS_TRUE : EOS_FALSE;

	UE_LOG_ONLINE_SESSION(Log, TEXT("EOS_SessionModification_SetJoinInProgressAllowed() set to (%s) for session (%s)"), *LexToString(Options.bAllowJoinInProgress), *Session->SessionName.ToString());

	EOS_EResult ResultCode = EOS_SessionModification_SetJoinInProgressAllowed(SessionModHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("EOS_SessionModification_SetJoinInProgressAllowed() failed with EOS result code (%s)"), *LexToString(ResultCode));
	}
}

void FOnlineSessionEOS::AddAttribute(EOS_HSessionModification SessionModHandle, const EOS_Sessions_AttributeData* Attribute)
{
	EOS_SessionModification_AddAttributeOptions Options = { };
	Options.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONMODIFICATION_ADDATTRIBUTE_API_LATEST, 2);
	Options.AdvertisementType = EOS_ESessionAttributeAdvertisementType::EOS_SAAT_Advertise;
	Options.SessionAttribute = Attribute;

	UE_LOG_ONLINE_SESSION(Log, TEXT("EOS_SessionModification_AddAttribute() named (%s) with value (%s)"), UTF8_TO_TCHAR(Attribute->Key), *MakeStringFromAttributeValue(Attribute));

	EOS_EResult ResultCode = EOS_SessionModification_AddAttribute(SessionModHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("EOS_SessionModification_AddAttribute() failed for attribute name (%s) with EOS result code (%s)"), *FString(Attribute->Key), *LexToString(ResultCode));
	}
}

void FOnlineSessionEOS::SetAttributes(EOS_HSessionModification SessionModHandle, FNamedOnlineSession* Session)
{
	FAttributeOptions Opt1("NumPrivateConnections", Session->SessionSettings.NumPrivateConnections);
	AddAttribute(SessionModHandle, &Opt1);

	FAttributeOptions Opt2("NumPublicConnections", Session->SessionSettings.NumPublicConnections);
	AddAttribute(SessionModHandle, &Opt2);

	FAttributeOptions Opt5("bAntiCheatProtected", Session->SessionSettings.bAntiCheatProtected);
	AddAttribute(SessionModHandle, &Opt5);

	FAttributeOptions Opt6("bUsesStats", Session->SessionSettings.bUsesStats);
	AddAttribute(SessionModHandle, &Opt6);

	FAttributeOptions Opt7("bIsDedicated", Session->SessionSettings.bIsDedicated);
	AddAttribute(SessionModHandle, &Opt7);

	FAttributeOptions Opt8("BuildUniqueId", Session->SessionSettings.BuildUniqueId);
	AddAttribute(SessionModHandle, &Opt8);

	// Add all of the session settings
	for (FSessionSettings::TConstIterator It(Session->SessionSettings.Settings); It; ++It)
	{
		const FName KeyName = It.Key();
		const FOnlineSessionSetting& Setting = It.Value();

		// Skip unsupported types or non session advertised settings
		if (Setting.AdvertisementType < EOnlineDataAdvertisementType::ViaOnlineService || !IsSessionSettingTypeSupported(Setting.Data.GetType()))
		{
			continue;
		}

		FAttributeOptions Attribute(TCHAR_TO_UTF8(*KeyName.ToString()), Setting.Data);
		AddAttribute(SessionModHandle, &Attribute);
	}
}

struct FBeginMetricsOptions :
	public EOS_Metrics_BeginPlayerSessionOptions
{
	char DisplayNameAnsi[EOS_OSS_STRING_BUFFER_LENGTH];
	char ServerIpAnsi[EOS_OSS_STRING_BUFFER_LENGTH];
	char SessionIdAnsi[EOS_OSS_STRING_BUFFER_LENGTH];
	char ExternalIdAnsi[EOS_OSS_STRING_BUFFER_LENGTH];

	FBeginMetricsOptions() :
		EOS_Metrics_BeginPlayerSessionOptions()
	{
		ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_METRICS_BEGINPLAYERSESSION_API_LATEST, 1);
		GameSessionId = SessionIdAnsi;
		DisplayName = DisplayNameAnsi;
		ServerIp = ServerIpAnsi;
		AccountId.External = ExternalIdAnsi;
	}
};

void FOnlineSessionEOS::BeginSessionAnalytics(FNamedOnlineSession* Session)
{
	const int32 LocalUserNum = EOSSubsystem->UserManager->GetDefaultLocalUser();
	const FOnlineUserPtr LocalUser = EOSSubsystem->UserManager->GetLocalOnlineUser(LocalUserNum);
	const EOS_EpicAccountId AccountId = EOSSubsystem->UserManager->GetLocalEpicAccountId(LocalUserNum);
	if (LocalUser.IsValid() && AccountId != nullptr)
	{
		TSharedPtr<const FOnlineSessionInfoEOS> SessionInfoEOS = StaticCastSharedPtr<const FOnlineSessionInfoEOS>(Session->SessionInfo);

		FBeginMetricsOptions Options;
		FCStringAnsi::Strncpy(Options.ServerIpAnsi, TCHAR_TO_UTF8(*SessionInfoEOS->HostAddr->ToString(false)), EOS_OSS_STRING_BUFFER_LENGTH);
		FString DisplayName = LocalUser->GetDisplayName();
		FCStringAnsi::Strncpy(Options.DisplayNameAnsi, TCHAR_TO_UTF8(*DisplayName), EOS_OSS_STRING_BUFFER_LENGTH);
		Options.AccountIdType = EOS_EMetricsAccountIdType::EOS_MAIT_Epic;
		Options.AccountId.Epic = AccountId;

		EOS_EResult Result = EOS_Metrics_BeginPlayerSession(EOSSubsystem->MetricsHandle, &Options);
		if (Result != EOS_EResult::EOS_Success)
		{
			UE_LOG_ONLINE_SESSION(Error, TEXT("EOS_Metrics_BeginPlayerSession() returned EOS result code (%s)"), *LexToString(Result));
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionEOS::BeginSessionAnalytics] EOS_Metrics_BeginPlayerSession was not called. Needed AccountId was invalid for LocalUserNum [%d]"), LocalUserNum);
	}
}

template<typename BaseStruct>
struct TNamedSessionOptions :
	public BaseStruct
{
	char SessionNameAnsi[EOS_OSS_STRING_BUFFER_LENGTH];

	TNamedSessionOptions(const char* InSessionNameAnsi)
		: BaseStruct()
	{
		FCStringAnsi::Strncpy(SessionNameAnsi, InSessionNameAnsi, EOS_OSS_STRING_BUFFER_LENGTH);
		this->SessionName = SessionNameAnsi;
	}
};

struct FSessionCreateOptions :
	public TNamedSessionOptions<EOS_Sessions_CreateSessionModificationOptions>
{
	FSessionCreateOptions(const char* InSessionNameAnsi) :
		TNamedSessionOptions<EOS_Sessions_CreateSessionModificationOptions>(InSessionNameAnsi)
	{
		ApiVersion = 5;
		UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_CREATESESSIONMODIFICATION_API_LATEST, 5);
	}
};

TSharedPtr<class FInternetAddr> GetHostAddress(const FString& EosHostAddress, const FOnlineSessionSettings& Settings)
{
	TSharedPtr<class FInternetAddr> Result;

	if (EosHostAddress.StartsWith(EOS_CONNECTION_URL_PREFIX, ESearchCase::IgnoreCase))
	{
		Result = ISocketSubsystem::Get(EOS_SOCKETSUBSYSTEM)->GetAddressFromString(EosHostAddress);
	}
	else
	{
		Result = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetAddressFromString(EosHostAddress);
		if (Result)
		{
			// Read port from settings, or fall back on DefaultPort.
			int32 Port = FURL::UrlConfig.DefaultPort;

			// TODO Settings.Get _should_ work, BUT even if you set it as int32, the round trip to EOS sets it to int64, and there is no FOnlineSessionSettings::Get overload for int64...
			// so tl;dr int attribute types are not stable in osseos and you have to check for all possible variations...
			if (const FOnlineSessionSetting* Setting = Settings.Settings.Find(SESSION_ATTR_SERVERPORT))
			{
				switch (Setting->Data.GetType())
				{
					case EOnlineKeyValuePairDataType::Int32:
					{
						Setting->Data.GetValue(Port);
						break;
					}
					case EOnlineKeyValuePairDataType::Int64:
					{
						int64 Port64;
						Setting->Data.GetValue(Port64);
						Port = Port64;
						break;
					}
					default: checkNoEntry();
				}
			}

			Result->SetPort(Port);
		}
	}

	return Result;
}

bool IsNetDriverEOS(const FName NetDriverName)
{
	if (GEngine)
	{
		const FNetDriverDefinition* Found = GEngine->NetDriverDefinitions.FindByPredicate([NetDriverName](const FNetDriverDefinition& Elem)
		{
			return Elem.DefName == NetDriverName;
		});

		return Found && Found->DriverClassName.ToString().Contains(TEXT("NetDriverEOS"));
	}

	UE_LOG_ONLINE_SESSION(Verbose, TEXT("GEngine not valid. Unable to check if NetDriver [%s] was EOS."), *NetDriverName.ToString());

	return false;
}

uint32 FOnlineSessionEOS::CreateEOSSession(int32 HostingPlayerNum, FNamedOnlineSession* Session)
{
	check(Session != nullptr);

	EOS_HSessionModification SessionModHandle = nullptr;

	// We set the custom parameter to transmit presence information
	Session->SessionSettings.Settings.Add(USES_PRESENCE_ATTRIBUTE_KEY, FOnlineSessionSetting(Session->SessionSettings.bUsesPresence, EOnlineDataAdvertisementType::ViaOnlineService));

	if (!Session->SessionSettings.bUsesPresence && (Session->SessionSettings.bAllowJoinViaPresence || Session->SessionSettings.bAllowJoinViaPresenceFriendsOnly))
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionSettings::bUsesPresence is set to false, bAllowJoinViaPresence and bAllowJoinViaPresenceFriendsOnly will be automatically set to false as well"));

		Session->SessionSettings.bAllowJoinViaPresence = false;
		Session->SessionSettings.bAllowJoinViaPresenceFriendsOnly = false;
	}

	FSessionCreateOptions Options(TCHAR_TO_UTF8(*Session->SessionName.ToString()));
	Options.MaxPlayers = Session->SessionSettings.NumPrivateConnections + Session->SessionSettings.NumPublicConnections;
	Options.LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(HostingPlayerNum);
	Options.bPresenceEnabled = Session->SessionSettings.bUsesPresence;
	const auto BucketIdUtf8 = StringCast<UTF8CHAR>(*GetBucketId(Session->SessionSettings));
	Options.BucketId = (const char*)BucketIdUtf8.Get();

	EOS_EResult ResultCode = EOS_Sessions_CreateSessionModification(EOSSubsystem->SessionsHandle, &Options, &SessionModHandle);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("%hs EOS_Sessions_CreateSessionModification failed Result=%s"), __FUNCTION__, *LexToString(ResultCode));
		return ONLINE_FAIL;
	}

	Session->SessionState = EOnlineSessionState::Creating;
	Session->bHosting = true;

	FString HostAddr;

	if (!IsRunningDedicatedServer() && IsNetDriverEOS(NAME_GameNetDriver))
	{
		FInternetAddrEOS TempAddr(Options.LocalUserId);
		HostAddr = TempAddr.ToString(true);
	}
	else
	{
		const bool bUseLocalIPs = FParse::Param(FCommandLine::Get(), TEXT("UseLocalIPs"));
		if (bUseLocalIPs)
		{
			bool bCanBindAll;
			HostAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalHostAddr(*GLog, bCanBindAll)->ToString(false);
		}
		else
		{
			// Leave HostAddr blank, we will skip calling SetHostAddress, and EOS will set it to our public IP.
		}

		// We're using IP so need to share which port.
		Session->SessionSettings.Set(SESSION_ATTR_SERVERPORT, FURL::UrlConfig.DefaultPort, EOnlineDataAdvertisementType::ViaOnlineService);
	}

	if (!HostAddr.IsEmpty())
	{
		// Setting the EOS Host Address
		char HostAddrAnsi[EOS_OSS_STRING_BUFFER_LENGTH];
		FCStringAnsi::Strncpy(HostAddrAnsi, TCHAR_TO_UTF8(*HostAddr), EOS_OSS_STRING_BUFFER_LENGTH);

		EOS_SessionModification_SetHostAddressOptions HostOptions = { };
		HostOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONMODIFICATION_SETHOSTADDRESS_API_LATEST, 1);
		// Expect URLs to look like "EOS:PUID:SocketName:Channel" and channel can be optional
		HostOptions.HostAddress = HostAddrAnsi;
		EOS_EResult HostResult = EOS_SessionModification_SetHostAddress(SessionModHandle, &HostOptions);
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("%hs EOS_SessionModification_SetHostAddress(%s) returned (%s)"), __FUNCTION__, *HostAddr, *LexToString(HostResult));
	}
	else
	{
		// We'll set HostAddr locally, but it'll be ignored on the EOS API side
		HostAddr = TEXT("127.0.0.1");

		UE_LOG_ONLINE_SESSION(Verbose, TEXT("%hs The server's public IP Address will be set as the Session's HostAddress."), __FUNCTION__);
	}


	// TODO why can't this use FUniqueNetIdEOSSession::EmptyId()?
	TSharedPtr<FOnlineSessionInfoEOS> SessionInfo = MakeShared<FOnlineSessionInfoEOS>(FOnlineSessionInfoEOS::Create(FUniqueNetIdEOSSession::Create(FString())));
	SessionInfo->HostAddr = GetHostAddress(HostAddr, Session->SessionSettings);
	Session->SessionInfo = SessionInfo;

	UE_LOG_ONLINE_SESSION(Verbose, TEXT("%hs The HostAddress used for this session will be %s"), __FUNCTION__, *SessionInfo->HostAddr->ToString(true));

	FUpdateSessionCallback* CallbackObj = new FUpdateSessionCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, SessionName = Session->SessionName](const EOS_Sessions_UpdateSessionCallbackInfo* Data)
	{
		bool bWasSuccessful = false;

		FNamedOnlineSession* Session = GetNamedSession(SessionName);
		if (Session)
		{
			bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success || Data->ResultCode == EOS_EResult::EOS_Sessions_OutOfSync;
			if (bWasSuccessful)
			{
				TSharedPtr<FOnlineSessionInfoEOS> SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoEOS>(Session->SessionInfo);
				if (SessionInfo.IsValid())
				{
					SessionInfo->SessionId = FUniqueNetIdEOSSession::Create(UTF8_TO_TCHAR(Data->SessionId));
				}

				Session->SessionState = EOnlineSessionState::Pending;
				BeginSessionAnalytics(Session);

				RegisterLocalPlayers(Session);
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Error, TEXT("EOS_Sessions_UpdateSession() failed with EOS result code (%s)"), *LexToString(Data->ResultCode));

				Session->SessionState = EOnlineSessionState::NoSession;

				RemoveNamedSession(SessionName);
			}
		}

		TriggerOnCreateSessionCompleteDelegates(SessionName, bWasSuccessful);
	};

	return SharedSessionUpdate(SessionModHandle, Session, CallbackObj);
}

uint32 FOnlineSessionEOS::SharedSessionUpdate(EOS_HSessionModification SessionModHandle, FNamedOnlineSession* Session, FUpdateSessionCallback* Callback)
{
	// Set joinability flags
	SetPermissionLevel(SessionModHandle, Session);
	// Set max players
	SetMaxPlayers(SessionModHandle, Session);
	// Set invite flags
	SetInvitesAllowed(SessionModHandle, Session);
	// Set JIP flag
	SetJoinInProgress(SessionModHandle, Session);
	// Add any attributes for filtering by searchers
	SetAttributes(SessionModHandle, Session);

	// Commit the session changes
	EOS_Sessions_UpdateSessionOptions CreateOptions = { };
	CreateOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_UPDATESESSION_API_LATEST, 1);
	CreateOptions.SessionModificationHandle = SessionModHandle;
	EOS_Sessions_UpdateSession(EOSSubsystem->SessionsHandle, &CreateOptions, Callback, Callback->GetCallbackPtr());

	EOS_SessionModification_Release(SessionModHandle);

	return ONLINE_IO_PENDING;
}

bool FOnlineSessionEOS::StartSession(FName SessionName)
{
	uint32 Result = ONLINE_FAIL;
	// Grab the session information by name
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		// Can't start a match multiple times
		if (Session->SessionState == EOnlineSessionState::Pending ||
			Session->SessionState == EOnlineSessionState::Ended)
		{
			if (!Session->SessionSettings.bIsLANMatch)
			{
				if (Session->SessionSettings.bUseLobbiesIfAvailable)
				{
					Result = StartLobbySession(Session);
				}
				else
				{
					Result = StartEOSSession(Session);
				}
			}
			else
			{
				// If this lan match has join in progress disabled, shut down the beacon
				if (!Session->SessionSettings.bAllowJoinInProgress)
				{
					LANSession->StopLANSession();
				}
				Result = ONLINE_SUCCESS;
				Session->SessionState = EOnlineSessionState::InProgress;
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't start an online session (%s) in state %s"),
				*SessionName.ToString(),
				EOnlineSessionState::ToString(Session->SessionState));
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't start an online game for session (%s) that hasn't been created"), *SessionName.ToString());
	}

	if (Result != ONLINE_IO_PENDING)
	{
		EOSSubsystem->ExecuteNextTick([this, SessionName, Result]()
			{
				TriggerOnStartSessionCompleteDelegates(SessionName, Result == ONLINE_SUCCESS);
			});
	}

	return true;
}

struct FSessionStartOptions :
	public TNamedSessionOptions<EOS_Sessions_StartSessionOptions>
{
	FSessionStartOptions(const char* InSessionNameAnsi) :
		TNamedSessionOptions<EOS_Sessions_StartSessionOptions>(InSessionNameAnsi)
	{
		ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_STARTSESSION_API_LATEST, 1);
	}
};

typedef TEOSCallback<EOS_Sessions_OnStartSessionCallback, EOS_Sessions_StartSessionCallbackInfo, FOnlineSessionEOS> FStartSessionCallback;

uint32 FOnlineSessionEOS::StartEOSSession(FNamedOnlineSession* Session)
{
	Session->SessionState = EOnlineSessionState::Starting;

	FSessionStartOptions Options(TCHAR_TO_UTF8(*Session->SessionName.ToString()));
	FStartSessionCallback* CallbackObj = new FStartSessionCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, SessionName = Session->SessionName](const EOS_Sessions_StartSessionCallbackInfo* Data)
	{
		bool bWasSuccessful = false;

		if (FNamedOnlineSession* Session = GetNamedSession(SessionName))
		{
			Session->SessionState = EOnlineSessionState::InProgress;

			bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
			if (!bWasSuccessful)
			{
				UE_LOG_ONLINE_SESSION(Error, TEXT("EOS_Sessions_StartSession() failed with EOS result code (%s)"), *LexToString(Data->ResultCode));
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("Session [%s] not found"), *SessionName.ToString());
		}

		TriggerOnStartSessionCompleteDelegates(SessionName, bWasSuccessful);
	};

	EOS_Sessions_StartSession(EOSSubsystem->SessionsHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());

	return ONLINE_IO_PENDING;
}

uint32 FOnlineSessionEOS::StartLobbySession(FNamedOnlineSession* Session)
{
	Session->SessionState = EOnlineSessionState::Starting;

	EOSSubsystem->ExecuteNextTick([this, SessionName = Session->SessionName]()
	{
		if (FNamedOnlineSession* Session = GetNamedSession(SessionName))
		{
			Session->SessionState = EOnlineSessionState::InProgress;
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("Session [%s] not found"), *SessionName.ToString());
		}

		TriggerOnStartSessionCompleteDelegates(SessionName, true);
	});

	return ONLINE_IO_PENDING;
}

bool FOnlineSessionEOS::UpdateSession(FName SessionName, FOnlineSessionSettings& UpdatedSessionSettings, bool bShouldRefreshOnlineData)
{
	int32 Result = ONLINE_FAIL;

	// Grab the session information by name
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		Session->SessionSettings = UpdatedSessionSettings;

		if (!Session->SessionSettings.bIsLANMatch)
		{
			if (Session->SessionSettings.bUseLobbiesIfAvailable)
			{
				Result = UpdateLobbySession(Session,
					FOnUpdateSessionCompleteDelegate::CreateLambda([this](FName SessionName, bool bWasSuccessful)
					{
						TriggerOnUpdateSessionCompleteDelegates(SessionName, bWasSuccessful);
					}));
			}
			else
			{
				Result = UpdateEOSSession(Session);
			}
		}
		else
		{
			Result = ONLINE_SUCCESS;
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("No session (%s) found for update!"), *SessionName.ToString());
	}

	if (Result != ONLINE_IO_PENDING)
	{
		EOSSubsystem->ExecuteNextTick([this, SessionName, Result]()
			{
				TriggerOnUpdateSessionCompleteDelegates(SessionName, Result == ONLINE_SUCCESS);
			});
	}

	return true;
}

struct FSessionUpdateOptions :
	public TNamedSessionOptions<EOS_Sessions_UpdateSessionModificationOptions>
{
	FSessionUpdateOptions(const char* InSessionNameAnsi) :
		TNamedSessionOptions<EOS_Sessions_UpdateSessionModificationOptions>(InSessionNameAnsi)
	{
		ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_UPDATESESSIONMODIFICATION_API_LATEST, 1);
	}
};

uint32 FOnlineSessionEOS::UpdateEOSSession(FNamedOnlineSession* Session)
{
	if (Session->SessionState == EOnlineSessionState::Creating)
	{
		return ONLINE_IO_PENDING;
	}

	EOS_HSessionModification SessionModHandle = NULL;
	FSessionUpdateOptions Options(TCHAR_TO_UTF8(*Session->SessionName.ToString()));

	EOS_EResult ResultCode = EOS_Sessions_UpdateSessionModification(EOSSubsystem->SessionsHandle, &Options, &SessionModHandle);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("EOS_Sessions_UpdateSessionModification() failed with EOS result code (%s)"), *LexToString(ResultCode));
		return ONLINE_FAIL;
	}

	FUpdateSessionCallback* CallbackObj = new FUpdateSessionCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, SessionName = Session->SessionName](const EOS_Sessions_UpdateSessionCallbackInfo* Data)
	{
		bool bWasSuccessful = false;
		
		if (FNamedOnlineSession* Session = GetNamedSession(SessionName))
		{
			bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success || Data->ResultCode == EOS_EResult::EOS_Sessions_OutOfSync;
			if (!bWasSuccessful)
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("EOS_Sessions_UpdateSession() failed with EOS result code (%s)"), *LexToString(Data->ResultCode));
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("Session [%s] not found"), *SessionName.ToString());
		}

		TriggerOnUpdateSessionCompleteDelegates(SessionName, bWasSuccessful);
	};

	return SharedSessionUpdate(SessionModHandle, Session, CallbackObj);
}

bool FOnlineSessionEOS::EndSession(FName SessionName)
{
	uint32 Result = ONLINE_FAIL;

	// Grab the session information by name
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		// Can't end a match that isn't in progress
		if (Session->SessionState == EOnlineSessionState::InProgress)
		{
			if (!Session->SessionSettings.bIsLANMatch)
			{
				if (Session->SessionSettings.bUseLobbiesIfAvailable)
				{
					Result = EndLobbySession(Session);
				}
				else
				{
					Result = EndEOSSession(Session);
				}
			}
			else
			{
				// If the session should be advertised and the lan beacon was destroyed, recreate
				if (Session->SessionSettings.bShouldAdvertise &&
					!LANSession.IsValid() &&
					LANSession->LanBeacon == nullptr &&
					EOSSubsystem->IsServer())
				{
					// Recreate the beacon
					Result = CreateLANSession(Session->HostingPlayerNum, Session);
				}
				else
				{
					Result = ONLINE_SUCCESS;
				}
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't end session (%s) in state %s"),
				*SessionName.ToString(),
				EOnlineSessionState::ToString(Session->SessionState));
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't end an online game for session (%s) that hasn't been created"),
			*SessionName.ToString());
	}

	if (Result != ONLINE_IO_PENDING)
	{
		EOSSubsystem->ExecuteNextTick([this, Session, SessionName, Result]()
			{
				if (Session)
				{
					Session->SessionState = EOnlineSessionState::Ended;
				}

				TriggerOnEndSessionCompleteDelegates(SessionName, Result == ONLINE_SUCCESS);
			});
	}

	return true;
}

struct FSessionEndOptions :
	public TNamedSessionOptions<EOS_Sessions_EndSessionOptions>
{
	FSessionEndOptions(const char* InSessionNameAnsi) :
		TNamedSessionOptions<EOS_Sessions_EndSessionOptions>(InSessionNameAnsi)
	{
		ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_ENDSESSION_API_LATEST, 1);
	}
};

typedef TEOSCallback<EOS_Sessions_OnEndSessionCallback, EOS_Sessions_EndSessionCallbackInfo, FOnlineSessionEOS> FEndSessionCallback;

uint32 FOnlineSessionEOS::EndEOSSession(FNamedOnlineSession* Session)
{
	// Only called from EndSession/DestroySession and presumes only in InProgress state
	check(Session && Session->SessionState == EOnlineSessionState::InProgress);

	Session->SessionState = EOnlineSessionState::Ending;

	FSessionEndOptions Options(TCHAR_TO_UTF8(*Session->SessionName.ToString()));
	FEndSessionCallback* CallbackObj = new FEndSessionCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, SessionName = Session->SessionName](const EOS_Sessions_EndSessionCallbackInfo* Data)
	{
		bool bWasSuccessful = false;

		if (FNamedOnlineSession* Session = GetNamedSession(SessionName))
		{
			Session->SessionState = EOnlineSessionState::Ended;

			bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
			if (!bWasSuccessful)
			{
				UE_LOG_ONLINE_SESSION(Error, TEXT("EOS_Sessions_EndSession() failed with EOS result code (%s)"), *LexToString(Data->ResultCode));
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("Session [%s] not found"), *SessionName.ToString());
		}

		TriggerOnEndSessionCompleteDelegates(SessionName, bWasSuccessful);
	};

	EOS_Sessions_EndSession(EOSSubsystem->SessionsHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());

	return ONLINE_IO_PENDING;
}

bool FOnlineSessionEOS::DestroySession(FName SessionName, const FOnDestroySessionCompleteDelegate& CompletionDelegate)
{
	uint32 Result = ONLINE_FAIL;

	// Find the session in question
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		if (Session->SessionState != EOnlineSessionState::Destroying)
		{
			if (!Session->SessionSettings.bIsLANMatch)
			{
				if (Session->SessionState == EOnlineSessionState::InProgress)
				{
					if (Session->SessionSettings.bUseLobbiesIfAvailable)
					{
						Result = EndLobbySession(Session);
					}
					else
					{
						Result = EndEOSSession(Session);
					}
				}

				if (Session->SessionSettings.bUseLobbiesIfAvailable)
				{
					Result = DestroyLobbySession(EOSSubsystem->UserManager->GetDefaultLocalUser(), Session, CompletionDelegate);
				}
				else
				{
					Result = DestroyEOSSession(Session, CompletionDelegate);
				}
			}
			else
			{
				if (LANSession.IsValid())
				{
					// Tear down the LAN beacon
					LANSession->StopLANSession();
					LANSession = nullptr;
				}

				Result = ONLINE_SUCCESS;
			}

			if (Result != ONLINE_IO_PENDING)
			{
				EOSSubsystem->ExecuteNextTick([this, CompletionDelegate, SessionName, Result]()
					{
						// The session info is no longer needed
						RemoveNamedSession(SessionName);
						CompletionDelegate.ExecuteIfBound(SessionName, Result == ONLINE_SUCCESS);
						TriggerOnDestroySessionCompleteDelegates(SessionName, Result == ONLINE_SUCCESS);
					});
			}
		}
		else
		{
			// Purposefully skip the delegate call as one should already be in flight
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Already in process of destroying session (%s)"), *SessionName.ToString());
		}
	}
	else
	{
		EOSSubsystem->ExecuteNextTick([this, CompletionDelegate, SessionName, Result]()
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't destroy a null online session (%s)"), *SessionName.ToString());
				CompletionDelegate.ExecuteIfBound(SessionName, false);
				TriggerOnDestroySessionCompleteDelegates(SessionName, false);
			});
	}

	return true;
}

struct FEndMetricsOptions :
	public EOS_Metrics_EndPlayerSessionOptions
{
	char ExternalIdAnsi[EOS_OSS_STRING_BUFFER_LENGTH];

	FEndMetricsOptions() :
		EOS_Metrics_EndPlayerSessionOptions()
	{
		ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_METRICS_ENDPLAYERSESSION_API_LATEST, 1);
		AccountId.External = ExternalIdAnsi;
	}
};

void FOnlineSessionEOS::EndSessionAnalytics()
{
	const int32 LocalUserNum = EOSSubsystem->UserManager->GetDefaultLocalUser();
	const EOS_EpicAccountId AccountId = EOSSubsystem->UserManager->GetLocalEpicAccountId(LocalUserNum);
	if (AccountId != nullptr)
	{
		FEndMetricsOptions Options;
		Options.AccountIdType = EOS_EMetricsAccountIdType::EOS_MAIT_Epic;
		Options.AccountId.Epic = AccountId;

		EOS_EResult Result = EOS_Metrics_EndPlayerSession(EOSSubsystem->MetricsHandle, &Options);
		if (Result != EOS_EResult::EOS_Success)
		{
			UE_LOG_ONLINE_SESSION(Error, TEXT("EOS_Metrics_EndPlayerSession() returned EOS result code (%s)"), *LexToString(Result));
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionEOS::EndSessionAnalytics] EOS_Metrics_EndPlayerSession was not called. Needed AccountId was invalid for LocalUserNum [%d]"), LocalUserNum);
	}
}

struct FSessionDestroyOptions :
	public TNamedSessionOptions<EOS_Sessions_DestroySessionOptions>
{
	FSessionDestroyOptions(const char* InSessionNameAnsi) :
		TNamedSessionOptions<EOS_Sessions_DestroySessionOptions>(InSessionNameAnsi)
	{
		ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_DESTROYSESSION_API_LATEST, 1);
	}
};

typedef TEOSCallback<EOS_Sessions_OnDestroySessionCallback, EOS_Sessions_DestroySessionCallbackInfo, FOnlineSessionEOS> FDestroySessionCallback;

uint32 FOnlineSessionEOS::DestroyEOSSession(FNamedOnlineSession* Session, const FOnDestroySessionCompleteDelegate& CompletionDelegate)
{
	Session->SessionState = EOnlineSessionState::Destroying;

	FSessionDestroyOptions Options(TCHAR_TO_UTF8(*Session->SessionName.ToString()));
	FDestroySessionCallback* CallbackObj = new FDestroySessionCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, SessionName = Session->SessionName, CompletionDelegate](const EOS_Sessions_DestroySessionCallbackInfo* Data)
	{
		EndSessionAnalytics();

		bool bWasSuccessful = false;
		if (FNamedOnlineSession* Session = GetNamedSession(SessionName))
		{
			Session->SessionState = EOnlineSessionState::NoSession;

			bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
			if (!bWasSuccessful)
			{
				UE_LOG_ONLINE_SESSION(Error, TEXT("EOS_Sessions_DestroySession() failed with EOS result code (%s)"), *LexToString(Data->ResultCode));
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("Session [%s] not found"), *SessionName.ToString());
		}

		RemoveNamedSession(SessionName);
		CompletionDelegate.ExecuteIfBound(SessionName, bWasSuccessful);
		TriggerOnDestroySessionCompleteDelegates(SessionName, bWasSuccessful);
	};

	EOS_Sessions_DestroySession(EOSSubsystem->SessionsHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());

	return ONLINE_IO_PENDING;
}

bool FOnlineSessionEOS::IsPlayerInSession(FName SessionName, const FUniqueNetId& UniqueId)
{
	return IsPlayerInSessionImpl(this, SessionName, UniqueId);
}

bool FOnlineSessionEOS::StartMatchmaking(const TArray< FUniqueNetIdRef >& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings, TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	EOSSubsystem->ExecuteNextTick([this, SessionName]()
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("StartMatchmaking is not supported on this platform. Use FindSessions or FindSessionById."));
			TriggerOnMatchmakingCompleteDelegates(SessionName, false);
		});

	return true;
}

bool FOnlineSessionEOS::CancelMatchmaking(int32 SearchingPlayerNum, FName SessionName)
{
	EOSSubsystem->ExecuteNextTick([this, SessionName]()
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("CancelMatchmaking is not supported on this platform. Use CancelFindSessions."));
			TriggerOnCancelMatchmakingCompleteDelegates(SessionName, false);
		});

	return true;
}

bool FOnlineSessionEOS::CancelMatchmaking(const FUniqueNetId& SearchingPlayerId, FName SessionName)
{
	EOSSubsystem->ExecuteNextTick([this, SessionName]()
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("CancelMatchmaking is not supported on this platform. Use CancelFindSessions."));
			TriggerOnCancelMatchmakingCompleteDelegates(SessionName, false);
		});

	return true;
}

bool FOnlineSessionEOS::FindSessions(int32 SearchingPlayerNum, const TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	uint32 Return = ONLINE_FAIL;

	// Don't start another search while one is in progress
	if (!CurrentSessionSearch.IsValid() || SearchSettings->SearchState != EOnlineAsyncTaskState::InProgress)
	{
		// LAN searching uses this as an approximation for ping so make sure to set it
		SessionSearchStartInSeconds = FPlatformTime::Seconds();

		// Free up previous results
		SearchSettings->SearchResults.Empty();
		// Copy the search pointer so we can keep it around
		CurrentSessionSearch = SearchSettings;

		// Check if its a LAN query
		if (!SearchSettings->bIsLanQuery)
		{
			bool bFindLobbies = false;
			if (SearchSettings->QuerySettings.Get(SEARCH_LOBBIES, bFindLobbies) && bFindLobbies)
			{
				Return = FindLobbySession(SearchingPlayerNum, SearchSettings);
			}
			else
			{
				Return = FindEOSSession(SearchingPlayerNum, SearchSettings);
			}
		}
		else
		{
			Return = FindLANSession();
		}

		if (Return == ONLINE_IO_PENDING)
		{
			SearchSettings->SearchState = EOnlineAsyncTaskState::InProgress;
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Ignoring game search request while another search is pending"));
		Return = ONLINE_IO_PENDING;
	}

	return Return == ONLINE_SUCCESS || Return == ONLINE_IO_PENDING;
}

bool FOnlineSessionEOS::FindSessions(const FUniqueNetId& SearchingPlayerId, const TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	// This function doesn't use the SearchingPlayerNum parameter, so passing in anything is fine.
	return FindSessions(EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(SearchingPlayerId), SearchSettings);
}

bool FOnlineSessionEOS::FindSessionById(const FUniqueNetId& SearchingUserId, const FUniqueNetId& SessionId, const FUniqueNetId& FriendId, const FOnSingleSessionResultCompleteDelegate& CompletionDelegate)
{
	bool bResult = false;

	// We create the search handle
	EOS_HLobbySearch LobbySearchHandle;
	EOS_Lobby_CreateLobbySearchOptions CreateLobbySearchOptions = { 0 };
	CreateLobbySearchOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_CREATELOBBYSEARCH_API_LATEST, 1);
	CreateLobbySearchOptions.MaxResults = EOS_SESSIONS_MAX_SEARCH_RESULTS;

	EOS_EResult CreateLobbySearchResult = EOS_Lobby_CreateLobbySearch(LobbyHandle, &CreateLobbySearchOptions, &LobbySearchHandle);
	if (CreateLobbySearchResult == EOS_EResult::EOS_Success)
	{
		const FTCHARToUTF8 Utf8LobbyId(*SessionId.ToString());
		// Set the lobby id we want to use to find lobbies			
		EOS_LobbySearch_SetLobbyIdOptions SetLobbyIdOptions = { 0 };
		SetLobbyIdOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYSEARCH_SETLOBBYID_API_LATEST, 1);
		SetLobbyIdOptions.LobbyId = (EOS_LobbyId)Utf8LobbyId.Get();

		EOS_LobbySearch_SetLobbyId(LobbySearchHandle, &SetLobbyIdOptions);

		// Then perform the search
		CurrentSessionSearch = MakeShareable(new FOnlineSessionSearch());
		CurrentSessionSearch->SearchState = EOnlineAsyncTaskState::InProgress;

		StartLobbySearch(EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(SearchingUserId), LobbySearchHandle, CurrentSessionSearch.ToSharedRef(),
			FOnSingleSessionResultCompleteDelegate::CreateLambda([this, OrigCallback = FOnSingleSessionResultCompleteDelegate(CompletionDelegate), SessId = FUniqueNetIdEOSSession::Create(SessionId.ToString())](int32 LocalUserNum, bool bWasSuccessful, const FOnlineSessionSearchResult& EOSResult)
		{
			if (bWasSuccessful)
			{
				OrigCallback.ExecuteIfBound(LocalUserNum, bWasSuccessful, EOSResult);
				return;
			}
			// Didn't find a lobby so search sessions
			FindEOSSessionById(LocalUserNum, *SessId, OrigCallback);
		}));

		bResult = true;
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::FindSessionById] CreateLobbySearch not successful. Finished with EOS_EResult %s"), *LexToString(CreateLobbySearchResult));
	}

	return bResult;
}

void FOnlineSessionEOS::AddSearchAttribute(EOS_HSessionSearch SearchHandle, const EOS_Sessions_AttributeData* Attribute, EOS_EOnlineComparisonOp ComparisonOp)
{
	EOS_SessionSearch_SetParameterOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONSEARCH_SETPARAMETER_API_LATEST, 1);
	Options.Parameter = Attribute;
	Options.ComparisonOp = ComparisonOp;

	EOS_EResult ResultCode = EOS_SessionSearch_SetParameter(SearchHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("EOS_SessionSearch_SetParameter() failed with EOS result code (%s)"), *LexToString(ResultCode));
	}
}

void FOnlineSessionEOS::AddLobbySearchAttribute(EOS_HLobbySearch LobbySearchHandle, const EOS_Lobby_AttributeData* Attribute, EOS_EOnlineComparisonOp ComparisonOp)
{
	EOS_LobbySearch_SetParameterOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYSEARCH_SETPARAMETER_API_LATEST, 1);
	Options.Parameter = Attribute;
	Options.ComparisonOp = ComparisonOp;

	EOS_EResult ResultCode = EOS_LobbySearch_SetParameter(LobbySearchHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("EOS_LobbySearch_SetParameter() failed with EOS result code (%s)"), *LexToString(ResultCode));
	}
}

void FOnlineSessionEOS::CopyAttributes(const FSessionDetailsEOS& SessionHandle, FOnlineSession& OutSession)
{
	EOS_SessionDetails_GetSessionAttributeCountOptions CountOptions = { };
	CountOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONDETAILS_GETSESSIONATTRIBUTECOUNT_API_LATEST, 1);
	int32 Count = EOS_SessionDetails_GetSessionAttributeCount(SessionHandle.SessionDetailsHandle, &CountOptions);

	for (int32 Index = 0; Index < Count; Index++)
	{
		EOS_SessionDetails_CopySessionAttributeByIndexOptions AttrOptions = { };
		AttrOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONDETAILS_COPYSESSIONATTRIBUTEBYINDEX_API_LATEST, 1);
		AttrOptions.AttrIndex = Index;

		EOS_SessionDetails_Attribute* Attribute = NULL;
		EOS_EResult ResultCode = EOS_SessionDetails_CopySessionAttributeByIndex(SessionHandle.SessionDetailsHandle, &AttrOptions, &Attribute);
		if (ResultCode == EOS_EResult::EOS_Success)
		{
			FString Key = Attribute->Data->Key;

			if (Key == USES_PRESENCE_ATTRIBUTE_KEY.ToString())
			{
				OutSession.SessionSettings.bUsesPresence = Attribute->Data->Value.AsBool == EOS_TRUE;
			}
			else if (Key == TEXT("NumPublicConnections"))
			{
				// Adjust the public connections based upon this
				OutSession.SessionSettings.NumPublicConnections = Attribute->Data->Value.AsInt64;
			}
			else if (Key == TEXT("NumPrivateConnections"))
			{
				// Adjust the private connections based upon this
				OutSession.SessionSettings.NumPrivateConnections = Attribute->Data->Value.AsInt64;
			}
			else if (Key == TEXT("bAntiCheatProtected"))
			{
				OutSession.SessionSettings.bAntiCheatProtected = Attribute->Data->Value.AsBool == EOS_TRUE;
			}
			else if (Key == TEXT("bUsesStats"))
			{
				OutSession.SessionSettings.bUsesStats = Attribute->Data->Value.AsBool == EOS_TRUE;
			}
			else if (Key == TEXT("bIsDedicated"))
			{
				OutSession.SessionSettings.bIsDedicated = Attribute->Data->Value.AsBool == EOS_TRUE;
			}
			else if (Key == TEXT("BuildUniqueId"))
			{
				OutSession.SessionSettings.BuildUniqueId = Attribute->Data->Value.AsInt64;
			}
			// Handle FOnlineSessionSetting settings
			else
			{
				FOnlineSessionSetting Setting;
				switch (Attribute->Data->ValueType)
				{
					case EOS_ESessionAttributeType::EOS_SAT_Boolean:
					{
						Setting.Data.SetValue(Attribute->Data->Value.AsBool == EOS_TRUE);
						break;
					}
					case EOS_ESessionAttributeType::EOS_SAT_Int64:
					{
						Setting.Data.SetValue(int64(Attribute->Data->Value.AsInt64));
						break;
					}
					case EOS_ESessionAttributeType::EOS_SAT_Double:
					{
						Setting.Data.SetValue(Attribute->Data->Value.AsDouble);
						break;
					}
					case EOS_ESessionAttributeType::EOS_SAT_String:
					{
						Setting.Data.SetValue(UTF8_TO_TCHAR(Attribute->Data->Value.AsUtf8));
						break;
					}
				}
				OutSession.SessionSettings.Settings.Add(FName(Key), Setting);
			}
		}

		EOS_SessionDetails_Attribute_Release(Attribute);
	}
}

void FOnlineSessionEOS::AddSearchResult(const TSharedRef<FSessionDetailsEOS>& SessionHandle, const TSharedRef<FOnlineSessionSearch>& SearchSettings, FOnCopySessionDataCompleteCallback&& Callback)
{
	EOS_SessionDetails_Info* EosSessionDetailsInfo = nullptr;
	EOS_SessionDetails_CopyInfoOptions CopyOptions = { };
	CopyOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONDETAILS_COPYINFO_API_LATEST, 1);
	EOS_EResult CopyResult = EOS_SessionDetails_CopyInfo(SessionHandle->SessionDetailsHandle, &CopyOptions, &EosSessionDetailsInfo);
	if (CopyResult == EOS_EResult::EOS_Success)
	{
		const int32 Position = SearchSettings->SearchResults.AddZeroed();
		FOnlineSessionSearchResult& SearchResult = SearchSettings->SearchResults[Position];

		// This will set the host address and port
		TSharedPtr<FOnlineSessionInfoEOS> OnlineSessionInfo = MakeShared<FOnlineSessionInfoEOS>(FOnlineSessionInfoEOS::Create(FUniqueNetIdEOSSession::Create(EosSessionDetailsInfo->SessionId), SessionHandle));
		SearchResult.Session.SessionInfo = OnlineSessionInfo;

		SearchResult.Session.NumOpenPrivateConnections = EosSessionDetailsInfo->NumOpenPublicConnections;
		SearchResult.Session.SessionSettings.NumPrivateConnections = EosSessionDetailsInfo->Settings->NumPublicConnections;
		SearchResult.Session.SessionSettings.bAllowJoinInProgress = EosSessionDetailsInfo->Settings->bAllowJoinInProgress == EOS_TRUE;
		SearchResult.Session.SessionSettings.bAllowInvites = EosSessionDetailsInfo->Settings->bInvitesAllowed == EOS_TRUE;

		switch (EosSessionDetailsInfo->Settings->PermissionLevel)
		{
			case EOS_EOnlineSessionPermissionLevel::EOS_OSPF_InviteOnly:
			{
				SearchResult.Session.SessionSettings.bAllowJoinViaPresence = false;
				break;
			}
			case EOS_EOnlineSessionPermissionLevel::EOS_OSPF_JoinViaPresence:
			case EOS_EOnlineSessionPermissionLevel::EOS_OSPF_PublicAdvertised:
			{
				SearchResult.Session.SessionSettings.bAllowJoinViaPresence = true;
				break;
			}
		}

		CopyAttributes(*SessionHandle, SearchResult.Session);

		if (EosSessionDetailsInfo->OwnerServerClientId)
		{
			SearchResult.Session.OwningUserId = FUniqueNetIdEOS::DedicatedServerId();

			// ResolveUniqueNetIds is an asynchronous operation, so in the cases where it's not called, we'll delay the execution of this callback to match the flow
			EOSSubsystem->ExecuteNextTick([Callback = MoveTemp(Callback)]()
				{
					Callback(true);
				});
		}
		else
		{
			EOSSubsystem->UserManager->ResolveUniqueNetId(EOSSubsystem->UserManager->GetDefaultLocalUser(), EosSessionDetailsInfo->OwnerUserId, [this, WeakSearchSettings = SearchSettings.ToWeakPtr(), Position, Callback = MoveTemp(Callback)](FUniqueNetIdEOSRef ResolvedUniqueNetId, const FOnlineError& Error)
			{
				if (TSharedPtr<FOnlineSessionSearch> StrongSearchSettings = WeakSearchSettings.Pin())
				{
					FOnlineSession& OutSession = StrongSearchSettings->SearchResults[Position].Session;
					OutSession.OwningUserId = ResolvedUniqueNetId;
					OutSession.OwningUserName = EOSSubsystem->UserManager->GetPlayerNickname(*ResolvedUniqueNetId);
				}
				Callback(true);
			});
		}

		// CopySearchResult above will populate the settings so we can now read the port and construct the HostAddress.
		OnlineSessionInfo->HostAddr = GetHostAddress(UTF8_TO_TCHAR(EosSessionDetailsInfo->HostAddress), SearchResult.Session.SessionSettings);

		EOS_SessionDetails_Info_Release(EosSessionDetailsInfo);
	}
	else
	{
		// CopySearchResult may launch an asynchronous operation, so we'll delay the execution of this callback to match the flow
		EOSSubsystem->ExecuteNextTick([CopyResult, Callback = MoveTemp(Callback)]()
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::AddSearchResult]SessionDetails_CopyInfo not successful. Finished with EOS_EResult %s"), *LexToString(CopyResult));

				Callback(false);
			});
	}
}

typedef TEOSCallback<EOS_SessionSearch_OnFindCallback, EOS_SessionSearch_FindCallbackInfo, FOnlineSessionEOS> FFindSessionsCallback;

uint32 FOnlineSessionEOS::FindEOSSession(int32 SearchingPlayerNum, const TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	EOS_HSessionSearch SearchHandle = nullptr;
	EOS_Sessions_CreateSessionSearchOptions HandleOptions = { };
	HandleOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_CREATESESSIONSEARCH_API_LATEST, 1);
	HandleOptions.MaxSearchResults = FMath::Clamp(SearchSettings->MaxSearchResults, 0, EOS_SESSIONS_MAX_SEARCH_RESULTS);

	EOS_EResult ResultCode = EOS_Sessions_CreateSessionSearch(EOSSubsystem->SessionsHandle, &HandleOptions, &SearchHandle);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("EOS_Sessions_CreateSessionSearch() failed with EOS result code (%s)"), *LexToString(ResultCode));
		return ONLINE_FAIL;
	}
	// Store our search handle for use/cleanup later
	CurrentSearchHandle = MakeShareable(new FSessionSearchEOS(SearchHandle));

	FAttributeOptions Opt1("NumPublicConnections", 1);
	AddSearchAttribute(SearchHandle, &Opt1, EOS_EOnlineComparisonOp::EOS_OCO_GREATERTHANOREQUAL);

	const auto BucketIdUtf8 = StringCast<UTF8CHAR>(*GetBucketId(*SearchSettings));
	FAttributeOptions Opt2(EOS_SESSIONS_SEARCH_BUCKET_ID, (const char*)BucketIdUtf8.Get());
	AddSearchAttribute(SearchHandle, &Opt2, EOS_EOnlineComparisonOp::EOS_OCO_EQUAL);

	// Add the search settings
	for (FSearchParams::TConstIterator It(SearchSettings->QuerySettings.SearchParams); It; ++It)
	{
		const FName Key = It.Key();
		const FOnlineSessionSearchParam& SearchParam = It.Value();

		// Game server keys are skipped
		if (Key == SEARCH_DEDICATED_ONLY || Key == SETTING_MAPNAME || Key == SEARCH_EMPTY_SERVERS_ONLY || Key == SEARCH_SECURE_SERVERS_ONLY || Key == SEARCH_LOBBIES)
		{
			continue;
		}

		if (!IsSessionSettingTypeSupported(SearchParam.Data.GetType()))
		{
			continue;
		}

#if UE_BUILD_DEBUG
		UE_LOG_ONLINE_SESSION(Log, TEXT("Adding search param named (%s), (%s)"), *Key.ToString(), *SearchParam.ToString());
#endif
		FString ParamName(Key.ToString());
		FAttributeOptions Attribute(TCHAR_TO_UTF8(*ParamName), SearchParam.Data);
		AddSearchAttribute(SearchHandle, &Attribute, ToEOSSearchOp(SearchParam.ComparisonOp));
	}

	FFindSessionsCallback* CallbackObj = new FFindSessionsCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, SearchSettings](const EOS_SessionSearch_FindCallbackInfo* Data)
	{
		bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
		if (bWasSuccessful)
		{
			EOS_SessionSearch_GetSearchResultCountOptions SearchResultOptions = { };
			SearchResultOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONSEARCH_GETSEARCHRESULTCOUNT_API_LATEST, 1);
			int32 SearchResultsCount = EOS_SessionSearch_GetSearchResultCount(CurrentSearchHandle->SearchHandle, &SearchResultOptions);

			if (SearchResultsCount > 0)
			{
				EOS_SessionSearch_CopySearchResultByIndexOptions IndexOptions = { };
				IndexOptions.ApiVersion = 1;
				UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONSEARCH_COPYSEARCHRESULTBYINDEX_API_LATEST, 1);
				for (int32 SessionIndex = 0; SessionIndex < SearchResultsCount; SessionIndex++)
				{
					EOS_HSessionDetails SessionDetailsHandle = nullptr;
					IndexOptions.SessionIndex = SessionIndex;
					EOS_EResult Result = EOS_SessionSearch_CopySearchResultByIndex(CurrentSearchHandle->SearchHandle, &IndexOptions, &SessionDetailsHandle);
					if (Result == EOS_EResult::EOS_Success)
					{
						const TSharedRef<FSessionDetailsEOS> SessionDetails = MakeShared<FSessionDetailsEOS>(SessionDetailsHandle);
						SessionSearchResultsPendingIdResolution.Add(SessionDetails);
						UE_LOG_ONLINE_SESSION(VeryVerbose, TEXT("[FOnlineSessionEOS::FindEOSSession] SessionSearch_CopySearchResultByIndex was successful."));
					}
					else
					{
						// It's unlikely EOS_SessionSearch_CopySearchResultByIndex would return failure. If it does return to the caller the failure and stop copying search results
						UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::FindEOSSession] SessionSearch_CopySearchResultByIndex not successful. Finished with EOS_EResult %s"), *LexToString(Data->ResultCode));
						TriggerOnFindSessionsCompleteDelegates(false);
						return; 
					}
				}

				// Make a copy to iterate over, as the AddSearchResult delegate removes entries.
				const TArray<TSharedRef<FSessionDetailsEOS>> SessionSearchResultsPendingIdResolutionCopy = SessionSearchResultsPendingIdResolution;
				
				// Need to keep track if there is a single failure in AddSearchResult. Assume true and only modify to false if adding a search results returns failure. 
				bAggregatedAddSearchResultSuccessful = true; 
				for (const TSharedRef<FSessionDetailsEOS>& SessionDetails : SessionSearchResultsPendingIdResolutionCopy)
				{
					AddSearchResult(SessionDetails, SearchSettings, [this, SessionDetails, SearchSettings, SearchResultsCount](bool bWasSuccessful)
						{
							SessionSearchResultsPendingIdResolution.Remove(SessionDetails);
							
							bAggregatedAddSearchResultSuccessful &= bWasSuccessful; 
							
							if (SessionSearchResultsPendingIdResolution.IsEmpty())
							{
								SearchSettings->SearchState = bAggregatedAddSearchResultSuccessful ? EOnlineAsyncTaskState::Done : EOnlineAsyncTaskState::Failed; 
								UE_LOG_ONLINE_SESSION(Log, TEXT("[FOnlineSessionEOS::FindEOSSession] SessionSearch returned %d results. Search state is: %s."), SearchResultsCount, bAggregatedAddSearchResultSuccessful ? TEXT("Done") : TEXT("Failed"));
								TriggerOnFindSessionsCompleteDelegates(bAggregatedAddSearchResultSuccessful);							
							}
						});
				}
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Log, TEXT("[FOnlineSessionEOS::FindEOSSession] SessionSearch_GetSearchResultCount returned no results"));
				TriggerOnFindSessionsCompleteDelegates(bWasSuccessful);
			}
		}
		else
		{
			SearchSettings->SearchState = EOnlineAsyncTaskState::Failed;
			UE_LOG_ONLINE_SESSION(Error, TEXT("EOS_SessionSearch_Find() failed with EOS result code (%s)"), *LexToString(Data->ResultCode));
			TriggerOnFindSessionsCompleteDelegates(bWasSuccessful);
		}
	};

	SearchSettings->SearchState = EOnlineAsyncTaskState::InProgress;

	// Execute the search
	EOS_SessionSearch_FindOptions Options = { };
	Options.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONSEARCH_FIND_API_LATEST, 2);
	Options.LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(SearchingPlayerNum);
	EOS_SessionSearch_Find(SearchHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());

	return ONLINE_IO_PENDING;
}

void FOnlineSessionEOS::FindEOSSessionById(int32 LocalUserNum, const FUniqueNetId& SessionId, const FOnSingleSessionResultCompleteDelegate& CompletionDelegate)
{
	EOS_HSessionSearch SearchHandle = nullptr;
	EOS_Sessions_CreateSessionSearchOptions HandleOptions = { };
	HandleOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_CREATESESSIONSEARCH_API_LATEST, 1);
	HandleOptions.MaxSearchResults = 1;

	EOS_EResult ResultCode = EOS_Sessions_CreateSessionSearch(EOSSubsystem->SessionsHandle, &HandleOptions, &SearchHandle);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("EOS_Sessions_CreateSessionSearch() failed with EOS result code (%s)"), *LexToString(ResultCode));
		CompletionDelegate.ExecuteIfBound(LocalUserNum, false, FOnlineSessionSearchResult());
		return;
	}

	const FTCHARToUTF8 Utf8SessionId(*SessionId.ToString());
	EOS_SessionSearch_SetSessionIdOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONSEARCH_SETSESSIONID_API_LATEST, 1);
	Options.SessionId = Utf8SessionId.Get();
	ResultCode = EOS_SessionSearch_SetSessionId(SearchHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("EOS_SessionSearch_SetSessionId() failed with EOS result code (%s)"), *LexToString(ResultCode));
		CompletionDelegate.ExecuteIfBound(LocalUserNum, false, FOnlineSessionSearchResult());
		return;
	}

	// Store our search handle for use/cleanup later
	CurrentSearchHandle = MakeShareable(new FSessionSearchEOS(SearchHandle));

	FFindSessionsCallback* CallbackObj = new FFindSessionsCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, LocalUserNum, OnComplete = FOnSingleSessionResultCompleteDelegate(CompletionDelegate)](const EOS_SessionSearch_FindCallbackInfo* Data)
	{
		TSharedRef<FOnlineSessionSearch> LocalSessionSearch = MakeShareable(new FOnlineSessionSearch());
		LocalSessionSearch->SearchState = EOnlineAsyncTaskState::InProgress;

		bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
		if (bWasSuccessful)
		{
			EOS_SessionSearch_GetSearchResultCountOptions SearchResultOptions = { };
			SearchResultOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONSEARCH_GETSEARCHRESULTCOUNT_API_LATEST, 1);
			int32 NumSearchResults = EOS_SessionSearch_GetSearchResultCount(CurrentSearchHandle->SearchHandle, &SearchResultOptions);

			// Only a single session is returned when using EOS_SessionSearch_SetSessionId
			EOS_SessionSearch_CopySearchResultByIndexOptions IndexOptions = { };
			IndexOptions.ApiVersion = 1;
			EOS_HSessionDetails SessionDetailsHandle = nullptr;
			IndexOptions.SessionIndex = 0;

			UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONSEARCH_COPYSEARCHRESULTBYINDEX_API_LATEST, 1);
			EOS_EResult Result = EOS_SessionSearch_CopySearchResultByIndex(CurrentSearchHandle->SearchHandle, &IndexOptions, &SessionDetailsHandle);
			if (Result == EOS_EResult::EOS_Success)
			{
				const TSharedRef<FSessionDetailsEOS> SessionDetails = MakeShared<FSessionDetailsEOS>(SessionDetailsHandle);
				AddSearchResult(SessionDetails, LocalSessionSearch, [this, LocalSessionSearch, LocalUserNum, OnComplete](bool bWasSuccessful)
					{
						OnComplete.ExecuteIfBound(LocalUserNum, bWasSuccessful, !LocalSessionSearch->SearchResults.IsEmpty() ? LocalSessionSearch->SearchResults.Last() : FOnlineSessionSearchResult());
					});
			}
			else
			{
				// It's unlikely EOS_SessionSearch_CopySearchResultByIndex would return failure. If it does return to the caller the failure.
				UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::FindEOSSession] SessionSearch_CopySearchResultByIndex not successful. Finished with EOS_EResult %s"), *LexToString(Data->ResultCode));
				LocalSessionSearch->SearchState = EOnlineAsyncTaskState::Failed;
				OnComplete.ExecuteIfBound(LocalUserNum, false, FOnlineSessionSearchResult());
			}
		}
		else
		{
			LocalSessionSearch->SearchState = EOnlineAsyncTaskState::Failed;
			UE_LOG_ONLINE_SESSION(Error, TEXT("EOS_SessionSearch_Find() failed with EOS result code (%s)"), *LexToString(Data->ResultCode));
			OnComplete.ExecuteIfBound(LocalUserNum, false, FOnlineSessionSearchResult());
		}
	};

	EOS_SessionSearch_FindOptions FindOptions = { };
	FindOptions.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONSEARCH_FIND_API_LATEST, 2);
	FindOptions.LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(LocalUserNum);

	EOS_SessionSearch_Find(SearchHandle, &FindOptions, CallbackObj, CallbackObj->GetCallbackPtr());
}

uint32 FOnlineSessionEOS::FindLANSession()
{
	uint32 Return = ONLINE_FAIL;

	if (!LANSession.IsValid())
	{
		LANSession = MakeShareable(new FLANSession());
	}

	// Recreate the unique identifier for this client
	GenerateNonce((uint8*)&LANSession->LanNonce, 8);

	FOnValidResponsePacketDelegate ResponseDelegate = FOnValidResponsePacketDelegate::CreateRaw(this, &FOnlineSessionEOS::OnValidResponsePacketReceived);
	FOnSearchingTimeoutDelegate TimeoutDelegate = FOnSearchingTimeoutDelegate::CreateRaw(this, &FOnlineSessionEOS::OnLANSearchTimeout);

	FNboSerializeToBufferEOS Packet(LAN_BEACON_MAX_PACKET_SIZE);
	LANSession->CreateClientQueryPacket(Packet, LANSession->LanNonce);
	if (LANSession->Search(Packet, ResponseDelegate, TimeoutDelegate))
	{
		Return = ONLINE_IO_PENDING;
	}

	if (Return == ONLINE_FAIL)
	{
		EOSSubsystem->ExecuteNextTick([this]()
			{
				CurrentSessionSearch->SearchState = EOnlineAsyncTaskState::Failed;

				// Just trigger the delegate as having failed
				TriggerOnFindSessionsCompleteDelegates(false);
			});
	}

	return Return;
}

bool FOnlineSessionEOS::CancelFindSessions()
{
	uint32 Return = ONLINE_FAIL;
	if (CurrentSessionSearch.IsValid() && CurrentSessionSearch->SearchState == EOnlineAsyncTaskState::InProgress)
	{
		// Make sure it's the right type
		if (CurrentSessionSearch->bIsLanQuery)
		{
			check(LANSession);
			Return = ONLINE_SUCCESS;
			LANSession->StopLANSession();
			CurrentSessionSearch->SearchState = EOnlineAsyncTaskState::Failed;
			CurrentSessionSearch = nullptr;
		}
		else
		{
			Return = ONLINE_SUCCESS;
			// NULLing out the object will prevent the async event from adding the results
			CurrentSessionSearch->SearchState = EOnlineAsyncTaskState::Failed;
			CurrentSessionSearch = nullptr;
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't cancel a search that isn't in progress"));
	}

	if (Return != ONLINE_IO_PENDING)
	{
		EOSSubsystem->ExecuteNextTick([this]()
			{
				TriggerOnCancelFindSessionsCompleteDelegates(true);
			});
	}

	return true;
}

bool FOnlineSessionEOS::JoinSession(int32 PlayerNum, FName SessionName, const FOnlineSessionSearchResult& DesiredSession)
{
	uint32 Return = ONLINE_FAIL;
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	// Don't join a session if already in one or hosting one
	if (Session == nullptr)
	{
		// Create a named session from the search result data
		Session = AddNamedSession(SessionName, DesiredSession.Session);
		Session->HostingPlayerNum = PlayerNum;

		// Create Internet or LAN match
		if (!Session->SessionSettings.bIsLANMatch)
		{
			if (DesiredSession.Session.SessionInfo.IsValid())
			{
				TSharedPtr<const FOnlineSessionInfoEOS> SearchSessionInfo = StaticCastSharedPtr<const FOnlineSessionInfoEOS>(DesiredSession.Session.SessionInfo);

				FOnlineSessionInfoEOS* NewSessionInfo = new FOnlineSessionInfoEOS(*SearchSessionInfo);
				Session->SessionInfo = MakeShareable(NewSessionInfo);

				if (DesiredSession.Session.SessionSettings.bUseLobbiesIfAvailable)
				{
					Return = JoinLobbySession(PlayerNum, Session, &DesiredSession.Session);
				}
				else
				{
					Return = JoinEOSSession(PlayerNum, Session);
				}
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Invalid session info on search result"));
			}
		}
		else
		{
			FOnlineSessionInfoEOS* NewSessionInfo = new FOnlineSessionInfoEOS();
			Session->SessionInfo = MakeShareable(NewSessionInfo);

			Return = JoinLANSession(PlayerNum, Session, &DesiredSession.Session);
		}

		if (Return != ONLINE_IO_PENDING)
		{
			if (Return != ONLINE_SUCCESS)
			{
				// Clean up the session info so we don't get into a confused state
				RemoveNamedSession(SessionName);
			}
			else
			{
				RegisterLocalPlayers(Session);
			}
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Session (%s) already exists, can't join twice"), *SessionName.ToString());
	}

	if (Return != ONLINE_IO_PENDING)
	{
		EOSSubsystem->ExecuteNextTick([this, SessionName, Return]()
			{
				// Just trigger the delegate as having failed
				TriggerOnJoinSessionCompleteDelegates(SessionName, Return == ONLINE_SUCCESS ? EOnJoinSessionCompleteResult::Success : EOnJoinSessionCompleteResult::UnknownError);
			});
	}

	return true;
}

bool FOnlineSessionEOS::JoinSession(const FUniqueNetId& SearchingUserId, FName SessionName, const FOnlineSessionSearchResult& DesiredSession)
{
	return JoinSession(EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(SearchingUserId), SessionName, DesiredSession);
}

struct FJoinSessionOptions :
	public TNamedSessionOptions<EOS_Sessions_JoinSessionOptions>
{
	FJoinSessionOptions(const char* InSessionNameAnsi) :
		TNamedSessionOptions<EOS_Sessions_JoinSessionOptions>(InSessionNameAnsi)
	{
		ApiVersion = 2;
		UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_JOINSESSION_API_LATEST, 2);
	}
};

typedef TEOSCallback<EOS_Sessions_OnJoinSessionCallback, EOS_Sessions_JoinSessionCallbackInfo, FOnlineSessionEOS> FJoinSessionCallback;

uint32 FOnlineSessionEOS::JoinEOSSession(int32 PlayerNum, FNamedOnlineSession* Session)
{
	if (!Session->SessionInfo.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("Session (%s) has invalid session info"), *Session->SessionName.ToString());
		return ONLINE_FAIL;
	}
	EOS_ProductUserId ProductUserId = EOSSubsystem->UserManager->GetLocalProductUserId(PlayerNum);
	if (ProductUserId == nullptr)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("Session (%s) invalid user id (%d)"), *Session->SessionName.ToString(), PlayerNum);
		return ONLINE_FAIL;
	}
	TSharedPtr<FOnlineSessionInfoEOS> EOSSessionInfo = StaticCastSharedPtr<FOnlineSessionInfoEOS>(Session->SessionInfo);
	if (!EOSSessionInfo->SessionId->IsValid())
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("Session (%s) has invalid session id"), *Session->SessionName.ToString());
		return ONLINE_FAIL;
	}

	Session->SessionState = EOnlineSessionState::Pending;

	FName SessionName = Session->SessionName;

	FJoinSessionCallback* CallbackObj = new FJoinSessionCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, SessionName](const EOS_Sessions_JoinSessionCallbackInfo* Data)
	{
		bool bWasSuccessful = false;

		FNamedOnlineSession* Session = GetNamedSession(SessionName);
		if (Session)
		{
			bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
			if (bWasSuccessful)
			{
				BeginSessionAnalytics(Session);
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Error, TEXT("EOS_Sessions_JoinSession() failed for session (%s) with EOS result code (%s)"), *SessionName.ToString(), *LexToString(Data->ResultCode));

				Session->SessionState = EOnlineSessionState::NoSession;

				RemoveNamedSession(SessionName);
			}
		}

		TriggerOnJoinSessionCompleteDelegates(SessionName, bWasSuccessful ? EOnJoinSessionCompleteResult::Success : EOnJoinSessionCompleteResult::UnknownError);
	};

	FJoinSessionOptions Options(TCHAR_TO_UTF8(*Session->SessionName.ToString()));
	Options.LocalUserId = ProductUserId;
	Options.SessionHandle = EOSSessionInfo->SessionHandle->SessionDetailsHandle;
	Options.bPresenceEnabled = Session->SessionSettings.bUsesPresence;
	EOS_Sessions_JoinSession(EOSSubsystem->SessionsHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());

	return ONLINE_IO_PENDING;
}

uint32 FOnlineSessionEOS::JoinLANSession(int32 PlayerNum, FNamedOnlineSession* Session, const FOnlineSession* SearchSession)
{
	uint32 Result = ONLINE_FAIL;
	Session->SessionState = EOnlineSessionState::Pending;

	if (Session->SessionInfo.IsValid())
	{
		// Copy the session info over
		TSharedPtr<const FOnlineSessionInfoEOS> SearchSessionInfo = StaticCastSharedPtr<const FOnlineSessionInfoEOS>(SearchSession->SessionInfo);
		TSharedPtr<FOnlineSessionInfoEOS> SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoEOS>(Session->SessionInfo);
		SessionInfo->HostAddr = SearchSessionInfo->HostAddr->Clone();
		Result = ONLINE_SUCCESS;
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Session (%s) has invalid session info"), *Session->SessionName.ToString());
	}

	return Result;
}

bool FOnlineSessionEOS::FindFriendSession(int32 LocalUserNum, const FUniqueNetId& Friend)
{
	bool bResult = false;

	// So far there is only a lobby implementation for this

	// We create the search handle
	EOS_HLobbySearch LobbySearchHandle;
	EOS_Lobby_CreateLobbySearchOptions CreateLobbySearchOptions = { 0 };
	CreateLobbySearchOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_CREATELOBBYSEARCH_API_LATEST, 1);
	CreateLobbySearchOptions.MaxResults = EOS_SESSIONS_MAX_SEARCH_RESULTS;

	EOS_EResult CreateLobbySearchResult = EOS_Lobby_CreateLobbySearch(LobbyHandle, &CreateLobbySearchOptions, &LobbySearchHandle);
	if (CreateLobbySearchResult == EOS_EResult::EOS_Success)
	{
		const FUniqueNetIdEOS& FriendEOSId = FUniqueNetIdEOS::Cast(Friend);

		// Set the user we wan to use to find lobbies
		EOS_LobbySearch_SetTargetUserIdOptions SetTargetUserIdOptions = { 0 };
		SetTargetUserIdOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYSEARCH_SETTARGETUSERID_API_LATEST, 1);
		SetTargetUserIdOptions.TargetUserId = FriendEOSId.GetProductUserId();

		EOS_LobbySearch_SetTargetUserId(LobbySearchHandle, &SetTargetUserIdOptions);

		// Then perform the search
		CurrentSessionSearch = MakeShareable(new FOnlineSessionSearch());
		CurrentSessionSearch->SearchState = EOnlineAsyncTaskState::InProgress;

		StartLobbySearch(LocalUserNum, LobbySearchHandle, CurrentSessionSearch.ToSharedRef(), FOnSingleSessionResultCompleteDelegate::CreateLambda([this](int32 LocalUserNum, bool bWasSuccessful, const FOnlineSessionSearchResult& EOSResult)
		{
			TriggerOnFindFriendSessionCompleteDelegates(LocalUserNum, bWasSuccessful, {EOSResult});
		}));

		bResult = true;
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::FindFriendSession] CreateLobbySearch not successful. Finished with EOS_EResult %s"), *LexToString(CreateLobbySearchResult));
		EOSSubsystem->ExecuteNextTick([this, LocalUserNum]()
		{
			TriggerOnFindFriendSessionCompleteDelegates(LocalUserNum, false, {});
		});
	}

	return bResult;
};

bool FOnlineSessionEOS::FindFriendSession(const FUniqueNetId& LocalUserId, const FUniqueNetId& Friend)
{
	return FindFriendSession(EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(LocalUserId), Friend);
}

bool FOnlineSessionEOS::FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList)
{
	EOSSubsystem->ExecuteNextTick([this, LocalUserIdRef = LocalUserId.AsShared()]()
		{
			// this function has to exist due to interface definition, but it does not have a meaningful implementation in EOS subsystem yet
			TriggerOnFindFriendSessionCompleteDelegates(EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(*LocalUserIdRef), false, {});
		});

	return true;
}

struct FSendSessionInviteOptions :
	public TNamedSessionOptions<EOS_Sessions_SendInviteOptions>
{
	FSendSessionInviteOptions(const char* InSessionNameAnsi) :
		TNamedSessionOptions<EOS_Sessions_SendInviteOptions>(InSessionNameAnsi)
	{
		ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_SENDINVITE_API_LATEST, 1);
	}
};

typedef TEOSCallback<EOS_Sessions_OnSendInviteCallback, EOS_Sessions_SendInviteCallbackInfo, FOnlineSessionEOS> FSendSessionInviteCallback;

bool FOnlineSessionEOS::SendSessionInvite(FName SessionName, EOS_ProductUserId SenderId, EOS_ProductUserId ReceiverId)
{
	bool bResult = false;

	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session != nullptr)
	{
		if (Session->SessionSettings.bUseLobbiesIfAvailable)
		{
			bResult = SendLobbyInvite(SessionName, SenderId, ReceiverId);
		}
		else
		{
			bResult = SendEOSSessionInvite(SessionName, SenderId, ReceiverId);
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::SendSessionInvite] Session with name %s not valid"), *SessionName.ToString());
	}

	return bResult;
}

bool FOnlineSessionEOS::SendLobbyInvite(FName SessionName, EOS_ProductUserId SenderId, EOS_ProductUserId ReceiverId)
{
	EOS_Lobby_SendInviteOptions SendInviteOptions = { 0 };
	SendInviteOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_SENDINVITE_API_LATEST, 1);
	const FTCHARToUTF8 Utf8LobbyId(*GetNamedSession(SessionName)->SessionInfo->GetSessionId().ToString());
	SendInviteOptions.LobbyId = (EOS_LobbyId)Utf8LobbyId.Get();
	SendInviteOptions.LocalUserId = SenderId;
	SendInviteOptions.TargetUserId = ReceiverId;
	
	FLobbySendInviteCallback* CallbackObj = new FLobbySendInviteCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	LobbySendInviteCallback = CallbackObj;
	CallbackObj->CallbackLambda = [this](const EOS_Lobby_SendInviteCallbackInfo* Data)
	{
		if (Data->ResultCode == EOS_EResult::EOS_Success)
		{
			UE_LOG_ONLINE_SESSION(Log, TEXT("[FOnlineSessionEOS::SendLobbyInvite] SendInvite was successful."));
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::SendLobbyInvite] SendInvite not successful. Finished with EOS_EResult %s"), *LexToString(Data->ResultCode));
		}
	};

	EOS_Lobby_SendInvite(LobbyHandle, &SendInviteOptions, CallbackObj, CallbackObj->GetCallbackPtr());

	return true;
}

bool FOnlineSessionEOS::SendEOSSessionInvite(FName SessionName, EOS_ProductUserId SenderId, EOS_ProductUserId ReceiverId)
{
	FSendSessionInviteOptions Options(TCHAR_TO_UTF8(*SessionName.ToString()));
	Options.LocalUserId = SenderId;
	Options.TargetUserId = ReceiverId;

	FSendSessionInviteCallback* CallbackObj = new FSendSessionInviteCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, SessionName](const EOS_Sessions_SendInviteCallbackInfo* Data)
	{
		bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
		if (!bWasSuccessful)
		{
			UE_LOG_ONLINE_SESSION(Error, TEXT("SendSessionInvite() failed for session (%s) with EOS result code (%s)"), *SessionName.ToString(), *LexToString(Data->ResultCode));
		}
	};

	EOS_Sessions_SendInvite(EOSSubsystem->SessionsHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
	
	return true;
}

bool FOnlineSessionEOS::SendSessionInviteToFriend(int32 LocalUserNum, FName SessionName, const FUniqueNetId& Friend)
{
	EOS_ProductUserId LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(LocalUserNum);
	if (LocalUserId == nullptr)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("SendSessionInviteToFriend() failed due to user (%d) being not logged in"), (int32)LocalUserNum);
		return false;
	}
	const FUniqueNetIdEOS& FriendEOSId = FUniqueNetIdEOS::Cast(Friend);
	const EOS_ProductUserId FriendId = FriendEOSId.GetProductUserId();
	if (EOS_ProductUserId_IsValid(FriendId) == EOS_FALSE)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("SendSessionInviteToFriend() failed due to target user (%s) having not played this game"), *Friend.ToDebugString());
		return false;
	}

	return SendSessionInvite(SessionName, LocalUserId, FriendId);
};

bool FOnlineSessionEOS::SendSessionInviteToFriend(const FUniqueNetId& LocalNetId, FName SessionName, const FUniqueNetId& Friend)
{
	const FUniqueNetIdEOS& LocalEOSId = FUniqueNetIdEOS::Cast(LocalNetId);
	const EOS_ProductUserId LocalUserId = LocalEOSId.GetProductUserId();
	if (EOS_ProductUserId_IsValid(LocalUserId) == EOS_FALSE)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("SendSessionInviteToFriend() failed due to user (%s) being not logged in"), *LocalNetId.ToDebugString());
		return false;
	}
	const FUniqueNetIdEOS& FriendEOSId = FUniqueNetIdEOS::Cast(Friend);
	const EOS_ProductUserId FriendId = FriendEOSId.GetProductUserId();
	if (EOS_ProductUserId_IsValid(FriendId) == EOS_FALSE)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("SendSessionInviteToFriend() failed due to target user (%s) having not played this game"), *Friend.ToDebugString());
		return false;
	}

	return SendSessionInvite(SessionName, LocalUserId, FriendId);
}

bool FOnlineSessionEOS::SendSessionInviteToFriends(int32 LocalUserNum, FName SessionName, const TArray< FUniqueNetIdRef >& Friends)
{
	for (const FUniqueNetIdRef& NetId : Friends)
	{
		if (SendSessionInviteToFriend(LocalUserNum, SessionName, *NetId) == false)
		{
			return false;
		}
	}
	return true;
};

bool FOnlineSessionEOS::SendSessionInviteToFriends(const FUniqueNetId& LocalUserId, FName SessionName, const TArray< FUniqueNetIdRef >& Friends)
{
	for (const FUniqueNetIdRef& NetId : Friends)
	{
		if (SendSessionInviteToFriend(LocalUserId, SessionName, *NetId) == false)
		{
			return false;
		}
	}
	return true;
}

bool FOnlineSessionEOS::PingSearchResults(const FOnlineSessionSearchResult& SearchResult)
{
	return false;
}

/** Get a resolved connection string from a session info */
static bool GetConnectStringFromSessionInfo(TSharedPtr<FOnlineSessionInfoEOS>& SessionInfo, FString& ConnectInfo, FName SocketNameOverride = NAME_None, int32 PortOverride = 0)
{
	if (!SessionInfo.IsValid() || !SessionInfo->HostAddr.IsValid())
	{
		return false;
	}

	FString AddrString = SessionInfo->HostAddr->ToString(false);
	if (AddrString.StartsWith(EOS_CONNECTION_URL_PREFIX, ESearchCase::IgnoreCase))
	{
		ConnectInfo = FString::Printf(TEXT("[%s]"), *AddrString);
	}
	else
	{
		int32 Port = PortOverride != 0 ? PortOverride : SessionInfo->HostAddr->GetPort();
		ConnectInfo = FString::Printf(TEXT("[%s]:%d"), *SessionInfo->HostAddr->ToString(false), Port);
	}

	if (SocketNameOverride != NAME_None)
	{
		ConnectInfo.ReplaceInline(*FName(NAME_GameNetDriver).ToString(), *SocketNameOverride.ToString());
	}

	return true;
}

bool FOnlineSessionEOS::GetResolvedConnectString(FName SessionName, FString& ConnectInfo, FName PortType)
{
	bool bSuccess = false;
	// Find the session
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session != nullptr)
	{
		TSharedPtr<FOnlineSessionInfoEOS> SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoEOS>(Session->SessionInfo);
		if (PortType == NAME_BeaconPort)
		{
			int32 BeaconListenPort = GetBeaconPortFromSessionSettings(Session->SessionSettings);
			bSuccess = GetConnectStringFromSessionInfo(SessionInfo, ConnectInfo, NAME_BeaconNetDriver, BeaconListenPort);
		}
		else if (PortType == NAME_GamePort)
		{
			bSuccess = GetConnectStringFromSessionInfo(SessionInfo, ConnectInfo);
		}

		if (!bSuccess)
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Invalid session info for session %s in GetResolvedConnectString()"), *SessionName.ToString());
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning,
			TEXT("Unknown session name (%s) specified to GetResolvedConnectString()"),
			*SessionName.ToString());
	}

	return bSuccess;
}

bool FOnlineSessionEOS::GetResolvedConnectString(const FOnlineSessionSearchResult& SearchResult, FName PortType, FString& ConnectInfo)
{
	bool bSuccess = false;
	if (SearchResult.Session.SessionInfo.IsValid())
	{
		TSharedPtr<FOnlineSessionInfoEOS> SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoEOS>(SearchResult.Session.SessionInfo);

		if (PortType == NAME_BeaconPort)
		{
			int32 BeaconListenPort = GetBeaconPortFromSessionSettings(SearchResult.Session.SessionSettings);
			bSuccess = GetConnectStringFromSessionInfo(SessionInfo, ConnectInfo, NAME_BeaconNetDriver, BeaconListenPort);
		}
		else if (PortType == NAME_GamePort)
		{
			bSuccess = GetConnectStringFromSessionInfo(SessionInfo, ConnectInfo);
		}
	}
	
	if (!bSuccess || ConnectInfo.IsEmpty())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Invalid session info in search result to GetResolvedConnectString()"));
	}

	return bSuccess;
}

FOnlineSessionSettings* FOnlineSessionEOS::GetSessionSettings(FName SessionName) 
{
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		return &Session->SessionSettings;
	}
	return nullptr;
}

void FOnlineSessionEOS::RegisterLocalPlayers(FNamedOnlineSession* Session)
{

}

void FOnlineSessionEOS::UpdateOrAddLobbyMember(const FUniqueNetIdEOSLobbyRef& LobbyNetId, const FUniqueNetIdEOSRef& PlayerId)
{
	if (FNamedOnlineSession* Session = GetNamedSessionFromLobbyId(*LobbyNetId))
	{
		// First we add the player to the session, if it wasn't already there
		bool bWasLobbyMemberAdded = false;
		if (!Session->SessionSettings.MemberSettings.Contains(PlayerId))
		{
			bWasLobbyMemberAdded = AddOnlineSessionMember(Session->SessionName, PlayerId);
		}

		if (FSessionSettings* MemberSettings = Session->SessionSettings.MemberSettings.Find(PlayerId))
		{
			const FTCHARToUTF8 Utf8LobbyId(*LobbyNetId->ToString());

			EOS_Lobby_CopyLobbyDetailsHandleOptions Options = {};
			Options.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_COPYLOBBYDETAILSHANDLE_API_LATEST, 1);
			Options.LobbyId = (EOS_LobbyId)Utf8LobbyId.Get();
			Options.LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(GetDefaultLocalUserForLobby(*LobbyNetId));

			EOS_HLobbyDetails LobbyDetailsHandle;

			EOS_EResult Result = EOS_Lobby_CopyLobbyDetailsHandle(LobbyHandle, &Options, &LobbyDetailsHandle);
			if (Result == EOS_EResult::EOS_Success)
			{
				FLobbyDetailsEOS LobbyDetails(LobbyDetailsHandle);

				// Then we update their attributes
				CopyLobbyMemberAttributes(LobbyDetails, PlayerId->GetProductUserId(), *MemberSettings);

				if (bWasLobbyMemberAdded)
				{
					TriggerOnSessionParticipantJoinedDelegates(Session->SessionName, *PlayerId);
				}
				else
				{
					TriggerOnSessionParticipantSettingsUpdatedDelegates(Session->SessionName, *PlayerId, Session->SessionSettings);
				}
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::UpdateOrAddLobbyMember] EOS_LobbyDetails_CopyLobbyDetailsHandle not successful. Finished with EOS_EResult %s"), *LexToString(Result));
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::UpdateOrAddLobbyMember] UniqueNetId %s not registered in the session's member settings."), *PlayerId->ToString());
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::UpdateOrAddLobbyMember] Unable to retrieve session with LobbyId %s"), *LobbyNetId->ToString());
	}
}

bool FOnlineSessionEOS::AddOnlineSessionMember(FName SessionName, const FUniqueNetIdRef& PlayerId)
{
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		if (!Session->SessionSettings.MemberSettings.Contains(PlayerId))
		{
			// update number of open connections
			if (Session->NumOpenPublicConnections > 0)
			{
				Session->NumOpenPublicConnections--;
			}
			else if (Session->NumOpenPrivateConnections > 0)
			{
				Session->NumOpenPrivateConnections--;
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::AddOnlineSessionMember] Could not add new member to session %s, no Public or Private connections open"), *SessionName.ToString());

				return false;
			}

			Session->SessionSettings.MemberSettings.Add(PlayerId, FSessionSettings());

			return true;
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::AddOnlineSessionMember] Could not find session with name: %s"), *SessionName.ToString());
	}

	return false;
}

bool FOnlineSessionEOS::RemoveOnlineSessionMember(FName SessionName, const FUniqueNetIdRef& PlayerId)
{
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		// update number of open connections
		if (Session->NumOpenPublicConnections < Session->SessionSettings.NumPublicConnections)
		{
			Session->NumOpenPublicConnections++;
		}
		else if (Session->NumOpenPrivateConnections < Session->SessionSettings.NumPrivateConnections)
		{
			Session->NumOpenPrivateConnections++;
		}

		Session->SessionSettings.MemberSettings.Remove(PlayerId);

		return true;
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::RemoveOnlineSessionMember] Could not find session with name: %s"), *SessionName.ToString());
	}

	return false;
}

bool FOnlineSessionEOS::RegisterPlayer(FName SessionName, const FUniqueNetId& PlayerId, bool bWasInvited)
{
	TArray< FUniqueNetIdRef > Players;
	Players.Add(PlayerId.AsShared());
	return RegisterPlayers(SessionName, Players, bWasInvited);
}

typedef TEOSCallback<EOS_Sessions_OnRegisterPlayersCallback, EOS_Sessions_RegisterPlayersCallbackInfo, FOnlineSessionEOS> FRegisterPlayersCallback;

bool FOnlineSessionEOS::RegisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasInvited)
{
	bool bSuccess = false;
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		TArray<EOS_ProductUserId> EOSIds;
		bSuccess = true;
		bool bRegisterEOS = !Session->SessionSettings.bUseLobbiesIfAvailable;

		for (int32 PlayerIdx=0; PlayerIdx<Players.Num(); PlayerIdx++)
		{
			const FUniqueNetIdRef& PlayerId = Players[PlayerIdx];
			const FUniqueNetIdEOS& PlayerEOSId = FUniqueNetIdEOS::Cast(*PlayerId);

			FUniqueNetIdMatcher PlayerMatch(*PlayerId);
			if (Session->RegisteredPlayers.IndexOfByPredicate(PlayerMatch) == INDEX_NONE)
			{
				Session->RegisteredPlayers.Add(PlayerId);
				if (bRegisterEOS)
				{
					EOSIds.Add(PlayerEOSId.GetProductUserId());
				}

				AddOnlineSessionMember(SessionName, PlayerId);
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Log, TEXT("Player %s already registered in session %s"), *PlayerId->ToDebugString(), *SessionName.ToString());
			}
		}

		if (bRegisterEOS && EOSIds.Num() > 0)
		{
			EOS_Sessions_RegisterPlayersOptions Options = { };
			Options.ApiVersion = 3;
			UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_REGISTERPLAYERS_API_LATEST, 3);
			Options.PlayersToRegister = EOSIds.GetData();
			Options.PlayersToRegisterCount = EOSIds.Num();
			const FTCHARToUTF8 Utf8SessionName(*SessionName.ToString());
			Options.SessionName = Utf8SessionName.Get();

			FRegisterPlayersCallback* CallbackObj = new FRegisterPlayersCallback(FOnlineSessionEOSWeakPtr(AsShared()));
			CallbackObj->CallbackLambda = [this, SessionName, RegisteredPlayers = TArray<FUniqueNetIdRef>(Players)](const EOS_Sessions_RegisterPlayersCallbackInfo* Data)
			{
				bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success || Data->ResultCode == EOS_EResult::EOS_NoChange;
				TriggerOnRegisterPlayersCompleteDelegates(SessionName, RegisteredPlayers, bWasSuccessful);
			};
			EOS_Sessions_RegisterPlayers(EOSSubsystem->SessionsHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
			return true;
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("No game present to join for session (%s)"), *SessionName.ToString());
	}

	EOSSubsystem->ExecuteNextTick([this, SessionName, RegisteredPlayers = TArray<FUniqueNetIdRef>(Players), bSuccess]()
	{
		TriggerOnRegisterPlayersCompleteDelegates(SessionName, RegisteredPlayers, bSuccess);
	});

	return true;
}

bool FOnlineSessionEOS::UnregisterPlayer(FName SessionName, const FUniqueNetId& PlayerId)
{
	TArray< FUniqueNetIdRef > Players;
	Players.Add(PlayerId.AsShared());
	return UnregisterPlayers(SessionName, Players);
}

typedef TEOSCallback<EOS_Sessions_OnUnregisterPlayersCallback, EOS_Sessions_UnregisterPlayersCallbackInfo, FOnlineSessionEOS> FUnregisterPlayersCallback;

bool FOnlineSessionEOS::UnregisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players)
{
	bool bSuccess = true;

	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		TArray<EOS_ProductUserId> EOSIds;
		bool bUnregisterEOS = !Session->SessionSettings.bUseLobbiesIfAvailable;
		for (int32 PlayerIdx=0; PlayerIdx < Players.Num(); PlayerIdx++)
		{
			const FUniqueNetIdRef& PlayerId = Players[PlayerIdx];
			const FUniqueNetIdEOS& PlayerEOSId = FUniqueNetIdEOS::Cast(*PlayerId);

			FUniqueNetIdMatcher PlayerMatch(*PlayerId);
			int32 RegistrantIndex = Session->RegisteredPlayers.IndexOfByPredicate(PlayerMatch);
			if (RegistrantIndex != INDEX_NONE)
			{
				Session->RegisteredPlayers.RemoveAtSwap(RegistrantIndex);
				if (bUnregisterEOS)
				{
					EOSIds.Add(PlayerEOSId.GetProductUserId());
				}

				RemoveOnlineSessionMember(SessionName, PlayerId);
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Verbose, TEXT("Player %s is not a registered player of session (%s)"), *PlayerId->ToDebugString(), *SessionName.ToString());
			}
		}
		if (bUnregisterEOS && EOSIds.Num() > 0)
		{
			EOS_Sessions_UnregisterPlayersOptions Options = { };
			Options.ApiVersion = 2;
			UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_UNREGISTERPLAYERS_API_LATEST, 2);
			Options.PlayersToUnregister = EOSIds.GetData();
			Options.PlayersToUnregisterCount = EOSIds.Num();
			const FTCHARToUTF8 Utf8SessionName(*SessionName.ToString());
			Options.SessionName = Utf8SessionName.Get();

			FUnregisterPlayersCallback* CallbackObj = new FUnregisterPlayersCallback(FOnlineSessionEOSWeakPtr(AsShared()));
			CallbackObj->CallbackLambda = [this, SessionName, UnregisteredPlayers = TArray<FUniqueNetIdRef>(Players)](const EOS_Sessions_UnregisterPlayersCallbackInfo* Data)
			{
				bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success || Data->ResultCode == EOS_EResult::EOS_NoChange;
				TriggerOnUnregisterPlayersCompleteDelegates(SessionName, UnregisteredPlayers, bWasSuccessful);
			};
			EOS_Sessions_UnregisterPlayers(EOSSubsystem->SessionsHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
			return true;
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("No game present to leave for session (%s)"), *SessionName.ToString());
		bSuccess = false;
	}

	EOSSubsystem->ExecuteNextTick([this, SessionName, Players, bSuccess]()
		{
			TriggerOnUnregisterPlayersCompleteDelegates(SessionName, Players, bSuccess);
		});

	return true;
}

void FOnlineSessionEOS::Tick(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_Session_Interface);
	TickLanTasks(DeltaTime);
}

void FOnlineSessionEOS::TickLanTasks(float DeltaTime)
{
	if (LANSession.IsValid() &&
		LANSession->GetBeaconState() > ELanBeaconState::NotUsingLanBeacon)
	{
		LANSession->Tick(DeltaTime);
	}
}

void FOnlineSessionEOS::AppendSessionToPacket(FNboSerializeToBufferEOS& Packet, FOnlineSession* Session)
{
	/** Owner of the session */
	((FNboSerializeToBuffer&)Packet) << Session->OwningUserId->ToString()
		<< Session->OwningUserName
		<< Session->NumOpenPrivateConnections
		<< Session->NumOpenPublicConnections;

	// Try to get the actual port the netdriver is using
	SetPortFromNetDriver(*EOSSubsystem, Session->SessionInfo);

	// Write host info (host addr, session id, and key)
	Packet << *StaticCastSharedPtr<FOnlineSessionInfoEOS>(Session->SessionInfo);

	// Now append per game settings
	AppendSessionSettingsToPacket(Packet, &Session->SessionSettings);
}

void FOnlineSessionEOS::AppendSessionSettingsToPacket(FNboSerializeToBufferEOS& Packet, FOnlineSessionSettings* SessionSettings)
{
#if DEBUG_LAN_BEACON
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("Sending session settings to client"));
#endif 

	// Members of the session settings class
	((FNboSerializeToBuffer&)Packet) << SessionSettings->NumPublicConnections
		<< SessionSettings->NumPrivateConnections
		<< (uint8)SessionSettings->bShouldAdvertise
		<< (uint8)SessionSettings->bIsLANMatch
		<< (uint8)SessionSettings->bIsDedicated
		<< (uint8)SessionSettings->bUsesStats
		<< (uint8)SessionSettings->bAllowJoinInProgress
		<< (uint8)SessionSettings->bAllowInvites
		<< (uint8)SessionSettings->bUsesPresence
		<< (uint8)SessionSettings->bAllowJoinViaPresence
		<< (uint8)SessionSettings->bAllowJoinViaPresenceFriendsOnly
		<< (uint8)SessionSettings->bAntiCheatProtected
	    << SessionSettings->BuildUniqueId;

	// First count number of advertised keys
	int32 NumAdvertisedProperties = 0;
	for (FSessionSettings::TConstIterator It(SessionSettings->Settings); It; ++It)
	{	
		const FOnlineSessionSetting& Setting = It.Value();
		if (Setting.AdvertisementType >= EOnlineDataAdvertisementType::ViaOnlineService)
		{
			NumAdvertisedProperties++;
		}
	}

	// Add count of advertised keys and the data
	((FNboSerializeToBuffer&)Packet) << (int32)NumAdvertisedProperties;
	for (FSessionSettings::TConstIterator It(SessionSettings->Settings); It; ++It)
	{
		const FOnlineSessionSetting& Setting = It.Value();
		if (Setting.AdvertisementType >= EOnlineDataAdvertisementType::ViaOnlineService)
		{
			((FNboSerializeToBuffer&)Packet) << It.Key();
			Packet << Setting;
#if DEBUG_LAN_BEACON
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("%s"), *Setting.ToString());
#endif
		}
	}
}

void FOnlineSessionEOS::OnValidQueryPacketReceived(uint8* PacketData, int32 PacketLength, uint64 ClientNonce)
{
	// Iterate through all registered sessions and respond for each LAN match
	FScopeLock ScopeLock(&SessionLock);
	for (int32 SessionIndex = 0; SessionIndex < Sessions.Num(); SessionIndex++)
	{
		FNamedOnlineSession* Session = &Sessions[SessionIndex];

		// Don't respond to query if the session is not a joinable LAN match.
		if (Session != nullptr)
		{
			const FOnlineSessionSettings& Settings = Session->SessionSettings;

			const bool bIsMatchInProgress = Session->SessionState == EOnlineSessionState::InProgress;

			const bool bIsMatchJoinable = Settings.bIsLANMatch &&
				(!bIsMatchInProgress || Settings.bAllowJoinInProgress) &&
				Settings.NumPublicConnections > 0;

			if (bIsMatchJoinable)
			{
				FNboSerializeToBufferEOS Packet(LAN_BEACON_MAX_PACKET_SIZE);
				// Create the basic header before appending additional information
				LANSession->CreateHostResponsePacket(Packet, ClientNonce);

				// Add all the session details
				AppendSessionToPacket(Packet, Session);

				// Broadcast this response so the client can see us
				LANSession->BroadcastPacket(Packet, Packet.GetByteCount());
			}
		}
	}
}

void FOnlineSessionEOS::ReadSessionFromPacket(FNboSerializeFromBufferEOS& Packet, FOnlineSession* Session)
{
#if DEBUG_LAN_BEACON
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("Reading session information from server"));
#endif

	/** Owner of the session */
	FString OwningUserIdStr;
	Packet >> OwningUserIdStr
		>> Session->OwningUserName
		>> Session->NumOpenPrivateConnections
		>> Session->NumOpenPublicConnections;

	Session->OwningUserId = FUniqueNetIdEOSRegistry::FindOrAdd(OwningUserIdStr);

	// Allocate and read the connection data
	FOnlineSessionInfoEOS* EOSSessionInfo = new FOnlineSessionInfoEOS();
	EOSSessionInfo->HostAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();
	Packet >> *EOSSessionInfo;
	Session->SessionInfo = MakeShareable(EOSSessionInfo); 

	// Read any per object data using the server object
	ReadSettingsFromPacket(Packet, Session->SessionSettings);
}

void FOnlineSessionEOS::ReadSettingsFromPacket(FNboSerializeFromBufferEOS& Packet, FOnlineSessionSettings& SessionSettings)
{
#if DEBUG_LAN_BEACON
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("Reading game settings from server"));
#endif

	// Clear out any old settings
	SessionSettings.Settings.Empty();

	// Members of the session settings class
	Packet >> SessionSettings.NumPublicConnections
		>> SessionSettings.NumPrivateConnections;
	uint8 Read = 0;
	// Read all the bools as bytes
	Packet >> Read;
	SessionSettings.bShouldAdvertise = !!Read;
	Packet >> Read;
	SessionSettings.bIsLANMatch = !!Read;
	Packet >> Read;
	SessionSettings.bIsDedicated = !!Read;
	Packet >> Read;
	SessionSettings.bUsesStats = !!Read;
	Packet >> Read;
	SessionSettings.bAllowJoinInProgress = !!Read;
	Packet >> Read;
	SessionSettings.bAllowInvites = !!Read;
	Packet >> Read;
	SessionSettings.bUsesPresence = !!Read;
	Packet >> Read;
	SessionSettings.bAllowJoinViaPresence = !!Read;
	Packet >> Read;
	SessionSettings.bAllowJoinViaPresenceFriendsOnly = !!Read;
	Packet >> Read;
	SessionSettings.bAntiCheatProtected = !!Read;

	// BuildId
	Packet >> SessionSettings.BuildUniqueId;

	// Now read the contexts and properties from the settings class
	int32 NumAdvertisedProperties = 0;
	// First, read the number of advertised properties involved, so we can presize the array
	Packet >> NumAdvertisedProperties;
	if (Packet.HasOverflow() == false)
	{
		FName Key;
		// Now read each context individually
		for (int32 Index = 0;
			Index < NumAdvertisedProperties && Packet.HasOverflow() == false;
			Index++)
		{
			FOnlineSessionSetting Setting;
			Packet >> Key;
			Packet >> Setting;
			SessionSettings.Set(Key, Setting);

#if DEBUG_LAN_BEACON
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("%s"), *Setting->ToString());
#endif
		}
	}
	
	// If there was an overflow, treat the string settings/properties as broken
	if (Packet.HasOverflow())
	{
		SessionSettings.Settings.Empty();
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("Packet overflow detected in ReadGameSettingsFromPacket()"));
	}
}

void FOnlineSessionEOS::OnValidResponsePacketReceived(uint8* PacketData, int32 PacketLength)
{
	// Create an object that we'll copy the data to
	FOnlineSessionSettings NewServer;
	if (CurrentSessionSearch.IsValid())
	{
		// Add space in the search results array
		FOnlineSessionSearchResult* NewResult = new (CurrentSessionSearch->SearchResults) FOnlineSessionSearchResult();
		// this is not a correct ping, but better than nothing
		NewResult->PingInMs = static_cast<int32>((FPlatformTime::Seconds() - SessionSearchStartInSeconds) * 1000);

		FOnlineSession* NewSession = &NewResult->Session;

		// Prepare to read data from the packet
		FNboSerializeFromBufferEOS Packet(PacketData, PacketLength);
		
		ReadSessionFromPacket(Packet, NewSession);

		// NOTE: we don't notify until the timeout happens
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to create new online game settings object"));
	}
}

void FOnlineSessionEOS::OnLANSearchTimeout()
{
	// See if there were any sessions that were marked as hosting before the search started
	bool bWasHosting = false;

	{
		FScopeLock ScopeLock(&SessionLock);
		for (int32 SessionIdx = 0; SessionIdx < Sessions.Num(); SessionIdx++)
		{
			FNamedOnlineSession& Session = Sessions[SessionIdx];
			if (Session.SessionSettings.bShouldAdvertise &&
				Session.SessionSettings.bIsLANMatch &&
				EOSSubsystem->IsServer())
			{
				bWasHosting = true;
				break;
			}
		}
	}

	if (bWasHosting)
	{
		FOnValidQueryPacketDelegate QueryPacketDelegate = FOnValidQueryPacketDelegate::CreateRaw(this, &FOnlineSessionEOS::OnValidQueryPacketReceived);
		// Maintain lan beacon if there was a session that was marked as hosting
		if (LANSession->Host(QueryPacketDelegate))
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to restart hosted LAN session after search completion"));
		}
	}
	else
	{
		// Stop future timeouts since we aren't searching any more
		LANSession->StopLANSession();
	}

	if (CurrentSessionSearch.IsValid())
	{
		if (CurrentSessionSearch->SearchResults.Num() > 0)
		{
			// Allow game code to sort the servers
			CurrentSessionSearch->SortSearchResults();
		}

		CurrentSessionSearch->SearchState = EOnlineAsyncTaskState::Done;

		CurrentSessionSearch = nullptr;
	}

	// Trigger the delegate as complete
	EOSSubsystem->ExecuteNextTick([this]()
		{
			TriggerOnFindSessionsCompleteDelegates(true);
		});
}

int32 FOnlineSessionEOS::GetNumSessions()
{
	FScopeLock ScopeLock(&SessionLock);
	return Sessions.Num();
}

void FOnlineSessionEOS::DumpSessionState()
{
	FScopeLock ScopeLock(&SessionLock);

	for (int32 SessionIdx=0; SessionIdx < Sessions.Num(); SessionIdx++)
	{
		DumpNamedSession(&Sessions[SessionIdx]);
	}
}

void FOnlineSessionEOS::RegisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnRegisterLocalPlayerCompleteDelegate& Delegate)
{
	Delegate.ExecuteIfBound(PlayerId, EOnJoinSessionCompleteResult::Success);
}

void FOnlineSessionEOS::UnregisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnUnregisterLocalPlayerCompleteDelegate& Delegate)
{
	Delegate.ExecuteIfBound(PlayerId, true);
}

void FOnlineSessionEOS::RemovePlayerFromSession(int32 LocalUserNum, FName SessionName, const FUniqueNetId& TargetPlayerId)
{
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		const FUniqueNetIdEOS& TargetPlayerEOSId = FUniqueNetIdEOS::Cast(TargetPlayerId);

		EOS_Lobby_KickMemberOptions KickMemberOptions = {};
		KickMemberOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_KICKMEMBER_API_LATEST, 1);
		const FTCHARToUTF8 Utf8LobbyId(*Session->SessionInfo->GetSessionId().ToString());
		KickMemberOptions.LobbyId = (EOS_LobbyId)Utf8LobbyId.Get();
		KickMemberOptions.LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(LocalUserNum);
		KickMemberOptions.TargetUserId = TargetPlayerEOSId.GetProductUserId();

		FLobbyRemovePlayerCallback* CallbackObj = new FLobbyRemovePlayerCallback(FOnlineSessionEOSWeakPtr(AsShared()));
		CallbackObj->CallbackLambda = [this](const EOS_Lobby_KickMemberCallbackInfo* Data)
		{
			if (Data->ResultCode == EOS_EResult::EOS_Success)
			{
				UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionEOS::RemovePlayerFromSession] KickMember finished successfully for lobby %s."), UTF8_TO_TCHAR(Data->LobbyId));
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::RemovePlayerFromSession] KickMember not successful. Finished with EOS_EResult %s"), *LexToString(Data->ResultCode));
			}
		};

		EOS_Lobby_KickMember(LobbyHandle, &KickMemberOptions, CallbackObj, CallbackObj->GetCallbackPtr());
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::RemovePlayerFromSession] Unable to retrieve session named %s"), *SessionName.ToString());
	}
}

void FOnlineSessionEOS::SetPortFromNetDriver(const FOnlineSubsystemEOS& Subsystem, const TSharedPtr<FOnlineSessionInfo>& SessionInfo)
{
	auto NetDriverPort = GetPortFromNetDriver(Subsystem.GetInstanceName());
	auto SessionInfoEOS = StaticCastSharedPtr<FOnlineSessionInfoEOS>(SessionInfo);
	if (SessionInfoEOS.IsValid() && SessionInfoEOS->HostAddr.IsValid())
	{
		FString AddrString = SessionInfoEOS->HostAddr->ToString(false);
		// Set port if IP address is used. EOS P2P does not use ports
		if (!AddrString.StartsWith(EOS_CONNECTION_URL_PREFIX, ESearchCase::IgnoreCase))
		{
			SessionInfoEOS->HostAddr->SetPort(NetDriverPort);
		}
	}
}

bool FOnlineSessionEOS::IsHost(const FNamedOnlineSession& Session) const
{
	if (EOSSubsystem->IsDedicated())
	{
		return true;
	}

	FUniqueNetIdPtr UserId = EOSSubsystem->UserManager->GetUniquePlayerId(Session.HostingPlayerNum);
	return (UserId.IsValid() && (*UserId == *Session.OwningUserId));
}

FUniqueNetIdPtr FOnlineSessionEOS::CreateSessionIdFromString(const FString& SessionIdStr)
{
	return FUniqueNetIdEOSSession::Create(SessionIdStr);
}

FString FOnlineSessionEOS::GetVoiceChatRoomName(int32 LocalUserNum, const FName& SessionName)
{
	FString RTCRoomNameStr;

	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session == nullptr)
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionEOS::GetRTCRoomNameFromLobbyId] Unable to find session with name %s"), *SessionName.ToString());
		return RTCRoomNameStr;
	}

	const FUniqueNetIdEOSLobby& LobbyId = FUniqueNetIdEOSLobby::Cast(Session->SessionInfo->GetSessionId());
	const auto LobbyIdUTF8 = StringCast<UTF8CHAR>(*LobbyId.ToString());	

	EOS_Lobby_GetRTCRoomNameOptions GetRTCRoomNameOptions = {};
	GetRTCRoomNameOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_GETRTCROOMNAME_API_LATEST, 1);
	GetRTCRoomNameOptions.LobbyId = (const char*)LobbyIdUTF8.Get();
	GetRTCRoomNameOptions.LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(LocalUserNum);

	char RTCRoomNameUTF8[256];
	uint32_t RTCRoomNameUTF8Length = UE_ARRAY_COUNT(RTCRoomNameUTF8);
	const EOS_EResult Result = EOS_Lobby_GetRTCRoomName(LobbyHandle, &GetRTCRoomNameOptions, RTCRoomNameUTF8, &RTCRoomNameUTF8Length);
	if (Result == EOS_EResult::EOS_Success)
	{
		RTCRoomNameStr = UTF8_TO_TCHAR(RTCRoomNameUTF8);
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionEOS::GetRTCRoomNameFromLobbyId] EOS_Lobby_GetRTCRoomName not successful. Finished with EOS_EResult %s"), *LexToString(Result));
	}

	return RTCRoomNameStr;
}

EOS_ELobbyPermissionLevel FOnlineSessionEOS::GetLobbyPermissionLevelFromSessionSettings(const FOnlineSessionSettings& SessionSettings)
{
	EOS_ELobbyPermissionLevel Result;

	if (SessionSettings.NumPublicConnections > 0)
	{
		Result = EOS_ELobbyPermissionLevel::EOS_LPL_PUBLICADVERTISED;
	}
	else if (SessionSettings.bAllowJoinViaPresence)
	{
		Result = EOS_ELobbyPermissionLevel::EOS_LPL_JOINVIAPRESENCE;
	}
	else
	{
		Result = EOS_ELobbyPermissionLevel::EOS_LPL_INVITEONLY;
	}

	return Result;
}

uint32_t FOnlineSessionEOS::GetLobbyMaxMembersFromSessionSettings(const FOnlineSessionSettings& SessionSettings)
{
	return SessionSettings.NumPrivateConnections + SessionSettings.NumPublicConnections;
}

FString FOnlineSessionEOS::GetBucketId(const FOnlineSessionSettings& SessionSettings)
{
	// Check if the Bucket Id custom setting is set, set default otherwise. EOS Sessions and Lobbies can not be created without it
	FString BucketIdStr;

	if (const FOnlineSessionSetting* BucketIdSetting = SessionSettings.Settings.Find(OSSEOS_BUCKET_ID_ATTRIBUTE_KEY))
	{
		BucketIdSetting->Data.GetValue(BucketIdStr);
	}
	else
	{
		const int32 BuildUniqueId = GetBuildUniqueId();

		UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionEOS::GetBucketId] 'OSSEOS_BUCKET_ID_ATTRIBUTE_KEY' (FString) Custom Setting needed to create EOS sessions not found. Setting \"%d\" as default."), BuildUniqueId);

		BucketIdStr = FString::FromInt(BuildUniqueId);
	}

	return BucketIdStr;
}

FString FOnlineSessionEOS::GetBucketId(const FOnlineSessionSearch& SessionSearch)
{
	// Check if the Bucket Id filter is set, set default otherwise.
	FString BucketIdStr;

	if (const FOnlineSessionSearchParam* BucketIdSetting = SessionSearch.QuerySettings.SearchParams.Find(OSSEOS_BUCKET_ID_ATTRIBUTE_KEY))
	{
		BucketIdSetting->Data.GetValue(BucketIdStr);
	}
	else
	{
		const int32 BuildUniqueId = GetBuildUniqueId();

		UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionEOS::GetBucketId] 'OSSEOS_BUCKET_ID_ATTRIBUTE_KEY' (FString) Custom Setting used to find EOS sessions not found. Setting \"%d\" as default."), BuildUniqueId);

		BucketIdStr = FString::FromInt(BuildUniqueId);
	}

	return BucketIdStr;
}

uint32 FOnlineSessionEOS::CreateLobbySession(int32 HostingPlayerNum, FNamedOnlineSession* Session)
{
	check(Session != nullptr);

	Session->SessionState = EOnlineSessionState::Creating;
	Session->bHosting = true;

	const EOS_ProductUserId LocalProductUserId = EOSSubsystem->UserManager->GetLocalProductUserId(HostingPlayerNum);
	const FUniqueNetIdPtr LocalUserNetId = EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(HostingPlayerNum);
	bool bUseHostMigration = true;
	Session->SessionSettings.Get(SETTING_HOST_MIGRATION, bUseHostMigration);

	if (!Session->SessionSettings.bUsesPresence && (Session->SessionSettings.bAllowJoinViaPresence || Session->SessionSettings.bAllowJoinViaPresenceFriendsOnly))
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionSettings::bUsesPresence is set to false, bAllowJoinViaPresence and bAllowJoinViaPresenceFriendsOnly will be automatically set to false as well"));

		Session->SessionSettings.bAllowJoinViaPresence = false;
		Session->SessionSettings.bAllowJoinViaPresenceFriendsOnly = false;
	}

	EOS_Lobby_CreateLobbyOptions CreateLobbyOptions = { 0 };
	CreateLobbyOptions.ApiVersion = 10;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_CREATELOBBY_API_LATEST, 10);
	CreateLobbyOptions.LocalUserId = LocalProductUserId;
	CreateLobbyOptions.MaxLobbyMembers = GetLobbyMaxMembersFromSessionSettings(Session->SessionSettings);
	CreateLobbyOptions.PermissionLevel = GetLobbyPermissionLevelFromSessionSettings(Session->SessionSettings);
	CreateLobbyOptions.bPresenceEnabled = Session->SessionSettings.bUsesPresence;
	CreateLobbyOptions.bAllowInvites = Session->SessionSettings.bAllowInvites;
	const auto BucketIdUtf8 = StringCast<UTF8CHAR>(*GetBucketId(Session->SessionSettings));
	CreateLobbyOptions.BucketId = (const char*)BucketIdUtf8.Get();
	CreateLobbyOptions.bDisableHostMigration = !bUseHostMigration;
#if WITH_EOS_RTC
	CreateLobbyOptions.bEnableRTCRoom = Session->SessionSettings.bUseLobbiesVoiceChatIfAvailable;
#endif
	const FTCHARToUTF8 Utf8SessionIdOverride(*Session->SessionSettings.SessionIdOverride);
	if (Session->SessionSettings.SessionIdOverride.Len() >= EOS_LOBBY_MIN_LOBBYIDOVERRIDE_LENGTH && Session->SessionSettings.SessionIdOverride.Len() <= EOS_LOBBY_MAX_LOBBYIDOVERRIDE_LENGTH)
	{
		CreateLobbyOptions.LobbyId = Utf8SessionIdOverride.Get();
	}
	else if (!Session->SessionSettings.SessionIdOverride.IsEmpty())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::CreateLobbySession] Session setting SessionIdOverride is of invalid length [%d]. Valid length range is between %d and %d."), Session->SessionSettings.SessionIdOverride.Len(), EOS_LOBBY_MIN_LOBBYIDOVERRIDE_LENGTH, EOS_LOBBY_MAX_LOBBYIDOVERRIDE_LENGTH)
	}

	CreateLobbyOptions.bEnableJoinById = false;
	CreateLobbyOptions.bRejoinAfterKickRequiresInvite = false;
	CreateLobbyOptions.bCrossplayOptOut = false;
	CreateLobbyOptions.RTCRoomJoinActionType = EOS_ELobbyRTCRoomJoinActionType::EOS_LRRJAT_AutomaticJoin;

	/*When the operation finishes, the EOS_Lobby_OnCreateLobbyCallback will run with an EOS_Lobby_CreateLobbyCallbackInfo data structure.
	If the data structure's ResultCode field indicates success, its LobbyId field contains the new lobby's ID value, which we will need to interact with the lobby further.*/

	FName SessionName = Session->SessionName;
	FLobbyCreatedCallback* CallbackObj = new FLobbyCreatedCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	LobbyCreatedCallback = CallbackObj;
	CallbackObj->CallbackLambda = [this, HostingPlayerNum, SessionName, LocalProductUserId, LocalUserNetId](const EOS_Lobby_CreateLobbyCallbackInfo* Data)
	{
		FNamedOnlineSession* Session = GetNamedSession(SessionName);
		if (Session)
		{
			bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
			if (bWasSuccessful)
			{
				UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionEOS::CreateLobbySession] CreateLobby was successful. LobbyId is %s."), UTF8_TO_TCHAR(Data->LobbyId));

				// Add the lobby owner to the member settings 
				AddOnlineSessionMember(SessionName, LocalUserNetId.ToSharedRef()); 
				
				Session->SessionState = EOnlineSessionState::Pending;

				TSharedPtr<FOnlineSessionInfoEOS> SessionInfo = MakeShared<FOnlineSessionInfoEOS>(FOnlineSessionInfoEOS::Create(FUniqueNetIdEOSLobby::Create(UTF8_TO_TCHAR(Data->LobbyId))));
				
				const FName NetDriverName = GetDefault<UNetDriverEOS>()->NetDriverName;
				SessionInfo->HostAddr = MakeShared<FInternetAddrEOS>(LocalProductUserId);

				Session->SessionInfo = SessionInfo;

#if WITH_EOSVOICECHAT
				if (FEOSVoiceChatUser* VoiceChatUser = static_cast<FEOSVoiceChatUser*>(EOSSubsystem->GetEOSVoiceChatUserInterface(*LocalUserNetId)))
				{
					VoiceChatUser->AddLobbyRoom(UTF8_TO_TCHAR(Data->LobbyId));
				}
#endif // WITH_EOSVOICECHAT

				BeginSessionAnalytics(Session);

				UpdateLobbySession(Session, FOnUpdateSessionCompleteDelegate::CreateThreadSafeSP(this, &FOnlineSessionEOS::OnCreateLobbySessionUpdateComplete, HostingPlayerNum));
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::CreateLobbySession] CreateLobby not successful. Finished with EOS_EResult %s"), *LexToString(Data->ResultCode));

				Session->SessionState = EOnlineSessionState::NoSession;

				RemoveNamedSession(SessionName);

				TriggerOnCreateSessionCompleteDelegates(SessionName, bWasSuccessful);
			}
		}
	};

	EOS_Lobby_CreateLobby(LobbyHandle, &CreateLobbyOptions, CallbackObj, CallbackObj->GetCallbackPtr());

	return ONLINE_IO_PENDING;
}

void FOnlineSessionEOS::OnCreateLobbySessionUpdateComplete(FName SessionName, bool bWasSuccessful, int32 HostingPlayerNum)
{
	if (!bWasSuccessful)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[OnCreateLobbySessionUpdateComplete] UpdateLobby not successful. Created session [%s] will be destroyed."), *SessionName.ToString());

		FNamedOnlineSession* Session = GetNamedSession(SessionName);
		if (Session)
		{
			DestroyLobbySessionOnCreationUpdateError(HostingPlayerNum, Session);
		}
	}
	else
	{
		TriggerOnCreateSessionCompleteDelegates(SessionName, bWasSuccessful);
	}
}

void FOnlineSessionEOS::DestroyLobbySessionOnCreationUpdateError(int32 LocalUserNum, FNamedOnlineSession* Session)
{
	check(Session != nullptr);
	check(Session->SessionInfo.IsValid());

	FOnlineSessionInfoEOS* SessionInfo = (FOnlineSessionInfoEOS*)(Session->SessionInfo.Get());
	check(Session->SessionSettings.bUseLobbiesIfAvailable); // We check if it's a lobby session

	EOS_Lobby_DestroyLobbyOptions DestroyOptions = { 0 };
	DestroyOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_DESTROYLOBBY_API_LATEST, 1);
	const FTCHARToUTF8 Utf8LobbyId(*SessionInfo->GetSessionId().ToString());
	DestroyOptions.LobbyId = (EOS_LobbyId)Utf8LobbyId.Get();
	DestroyOptions.LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(LocalUserNum);

	FName SessionName = Session->SessionName;
	FLobbyDestroyedCallback* DestroyCallbackObj = new FLobbyDestroyedCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	LobbyLeftCallback = DestroyCallbackObj;
	DestroyCallbackObj->CallbackLambda = [this, LocalUserNum, SessionName](const EOS_Lobby_DestroyLobbyCallbackInfo* Data)
	{
		FNamedOnlineSession* LobbySession = GetNamedSession(SessionName);
		if (LobbySession)
		{
			bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
			if (bWasSuccessful)
			{
				UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionEOS::DestroyLobbySessionOnCreationUpdateError] DestroyLobby was successful. LobbyId is %s."), UTF8_TO_TCHAR(Data->LobbyId));
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::DestroyLobbySessionOnCreationUpdateError] DestroyLobby not successful. Finished with EOS_EResult %s"), *LexToString(Data->ResultCode));
			}

#if WITH_EOSVOICECHAT
			if (FEOSVoiceChatUser* VoiceChatUser = static_cast<FEOSVoiceChatUser*>(EOSSubsystem->GetEOSVoiceChatUserInterface(*EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(LocalUserNum))))
			{
				VoiceChatUser->RemoveLobbyRoom(UTF8_TO_TCHAR(Data->LobbyId));
			}
#endif // WITH_EOSVOICECHAT

			EndSessionAnalytics();

			LobbySession->SessionState = EOnlineSessionState::NoSession;

			RemoveNamedSession(SessionName);
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::DestroyLobbySession] Unable to find session %s"), *SessionName.ToString());
		}

		// Whether Lobby destruction is successful or not, creation will always be failed at this point
		TriggerOnCreateSessionCompleteDelegates(SessionName, false);
	};

	EOS_Lobby_DestroyLobby(LobbyHandle, &DestroyOptions, DestroyCallbackObj, DestroyCallbackObj->GetCallbackPtr());
}

uint32 FOnlineSessionEOS::JoinLobbySession(int32 PlayerNum, FNamedOnlineSession* Session, const FOnlineSession* SearchSession)
{
	check(Session != nullptr);

	uint32 Result = ONLINE_FAIL;

	if (Session->SessionInfo.IsValid())
	{
		FOnlineSessionInfoEOS* EOSSessionInfo = (FOnlineSessionInfoEOS*)(Session->SessionInfo.Get());
		if (EOSSessionInfo->SessionId->IsValid())
		{
			// TODO why not just copy construct/assign?
			const FOnlineSessionInfoEOS* SearchSessionInfo = (const FOnlineSessionInfoEOS*)(SearchSession->SessionInfo.Get());
			EOSSessionInfo->HostAddr = SearchSessionInfo->HostAddr;
			EOSSessionInfo->SessionHandle = SearchSessionInfo->SessionHandle;
			EOSSessionInfo->LobbyHandle = SearchSessionInfo->LobbyHandle;
			EOSSessionInfo->SessionId = SearchSessionInfo->SessionId;

			Session->SessionState = EOnlineSessionState::Pending;
			
			// We retrieve the cached LobbyDetailsHandle and we start the join operation
			EOS_Lobby_JoinLobbyOptions JoinLobbyOptions = { 0 };
			JoinLobbyOptions.ApiVersion = 5;
			UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_JOINLOBBY_API_LATEST, 5);
			JoinLobbyOptions.LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(PlayerNum);
			JoinLobbyOptions.bPresenceEnabled = Session->SessionSettings.bUsesPresence;
 			JoinLobbyOptions.LobbyDetailsHandle = EOSSessionInfo->LobbyHandle->LobbyDetailsHandle;
			JoinLobbyOptions.LocalRTCOptions = nullptr;
			JoinLobbyOptions.bCrossplayOptOut = false;
			JoinLobbyOptions.RTCRoomJoinActionType = EOS_ELobbyRTCRoomJoinActionType::EOS_LRRJAT_AutomaticJoin;

			const FName SessionName = Session->SessionName;
			const FUniqueNetIdEOSPtr LocalUserNetId = EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(PlayerNum);

			FLobbyJoinedCallback* CallbackObj = new FLobbyJoinedCallback(FOnlineSessionEOSWeakPtr(AsShared()));
			LobbyJoinedCallback = CallbackObj;
			CallbackObj->CallbackLambda = [this, SessionName, LocalUserNetId](const EOS_Lobby_JoinLobbyCallbackInfo* Data)
			{
				FNamedOnlineSession* Session = GetNamedSession(SessionName);
				if (Session)
				{
					const bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
					if (bWasSuccessful)
					{
						UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionEOS::JoinLobbySession] JoinLobby was successful. LobbyId is %s."), UTF8_TO_TCHAR(Data->LobbyId));

						BeginSessionAnalytics(Session);

#if WITH_EOSVOICECHAT
						if (FEOSVoiceChatUser* VoiceChatUser = static_cast<FEOSVoiceChatUser*>(EOSSubsystem->GetEOSVoiceChatUserInterface(*LocalUserNetId)))
						{
							VoiceChatUser->AddLobbyRoom(UTF8_TO_TCHAR(Data->LobbyId));
						}
#endif // WITH_EOSVOICECHAT

						OnLobbyUpdateReceived(Data->LobbyId); // We could use LocalUserNetId here instead of the default local user for the session, but the end result should be the same
					}
					else
					{
						UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::JoinLobbySession] JoinLobby not successful. Finished with EOS_EResult %s"), *LexToString(Data->ResultCode));

						Session->SessionState = EOnlineSessionState::NoSession;

						RemoveNamedSession(SessionName);
					}

					TriggerOnJoinSessionCompleteDelegates(SessionName, bWasSuccessful ? EOnJoinSessionCompleteResult::Success : EOnJoinSessionCompleteResult::UnknownError);
				}
				else
				{
					UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::JoinLobbySession] Unable to find session %s"), *SessionName.ToString());
					TriggerOnJoinSessionCompleteDelegates(SessionName, EOnJoinSessionCompleteResult::SessionDoesNotExist);
				}
			};

			EOS_Lobby_JoinLobby(LobbyHandle, &JoinLobbyOptions, CallbackObj, CallbackObj->GetCallbackPtr());

			Result = ONLINE_IO_PENDING;
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::JoinLobbySession] SessionId not valid"));
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::JoinLobbySession] SessionInfo not valid."));
	}

	return Result;
}

void FOnlineSessionEOS::SetLobbyPermissionLevel(EOS_HLobbyModification LobbyModificationHandle, FNamedOnlineSession* Session)
{
	check(Session != nullptr);

	EOS_LobbyModification_SetPermissionLevelOptions Options = { 0 };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONMODIFICATION_SETPERMISSIONLEVEL_API_LATEST, 1);
	Options.PermissionLevel = GetLobbyPermissionLevelFromSessionSettings(Session->SessionSettings);

	EOS_EResult ResultCode = EOS_LobbyModification_SetPermissionLevel(LobbyModificationHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::SetLobbyPermissionLevel] LobbyModification_SetPermissionLevel not successful. Finished with EOS_EResult %s"), *LexToString(ResultCode));
	}
}

void FOnlineSessionEOS::SetLobbyMaxMembers(EOS_HLobbyModification LobbyModificationHandle, FNamedOnlineSession* Session)
{
	check(Session != nullptr);

	EOS_LobbyModification_SetMaxMembersOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYMODIFICATION_SETMAXMEMBERS_API_LATEST, 1);
	Options.MaxMembers = GetLobbyMaxMembersFromSessionSettings(Session->SessionSettings);

	EOS_EResult ResultCode = EOS_LobbyModification_SetMaxMembers(LobbyModificationHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::SetLobbyMaxMembers] LobbyModification_SetJoinInProgressAllowed not successful. Finished with EOS_EResult %s"), *LexToString(ResultCode));
	}
}

void FOnlineSessionEOS::AddLobbyAttribute(EOS_HLobbyModification LobbyModificationHandle, const EOS_Lobby_AttributeData* Attribute)
{
	EOS_LobbyModification_AddAttributeOptions Options = { };
	Options.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYMODIFICATION_ADDATTRIBUTE_API_LATEST, 2);
	Options.Visibility = EOS_ELobbyAttributeVisibility::EOS_LAT_PUBLIC;
	Options.Attribute = Attribute;

	EOS_EResult ResultCode = EOS_LobbyModification_AddAttribute(LobbyModificationHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("[FOnlineSessionEOS::AddLobbyAttribute] LobbyModification_AddAttribute for attribute name (%s) not successful. Finished with EOS_EResult %s"), *FString(Attribute->Key), *LexToString(ResultCode));
	}
}

void FOnlineSessionEOS::AddLobbyMemberAttribute(EOS_HLobbyModification LobbyModificationHandle, const EOS_Lobby_AttributeData* Attribute)
{
	EOS_LobbyModification_AddMemberAttributeOptions Options = { };
	Options.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYMODIFICATION_ADDMEMBERATTRIBUTE_API_LATEST, 2);
	Options.Visibility = EOS_ELobbyAttributeVisibility::EOS_LAT_PUBLIC;
	Options.Attribute = Attribute;

	EOS_EResult ResultCode = EOS_LobbyModification_AddMemberAttribute(LobbyModificationHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("[FOnlineSessionEOS::AddLobbyMemberAttribute] LobbyModification_AddMemberAttribute for attribute name (%s) not successful. Finished with EOS_EResult %s"), *FString(Attribute->Key), *LexToString(ResultCode));
	}
}

void FOnlineSessionEOS::SetLobbyAttributes(EOS_HLobbyModification LobbyModificationHandle, FNamedOnlineSession* Session)
{
	check(Session != nullptr);

	// Now the session settings
	const FLobbyAttributeOptions Opt1("NumPrivateConnections", Session->SessionSettings.NumPrivateConnections);
	AddLobbyAttribute(LobbyModificationHandle, &Opt1);

	const FLobbyAttributeOptions Opt2("NumPublicConnections", Session->SessionSettings.NumPublicConnections);
	AddLobbyAttribute(LobbyModificationHandle, &Opt2);

	const FLobbyAttributeOptions Opt5("bAntiCheatProtected", Session->SessionSettings.bAntiCheatProtected);
	AddLobbyAttribute(LobbyModificationHandle, &Opt5);

	const FLobbyAttributeOptions Opt6("bUsesStats", Session->SessionSettings.bUsesStats);
	AddLobbyAttribute(LobbyModificationHandle, &Opt6);

	// Likely unnecessary for lobbies
	const FLobbyAttributeOptions Opt7("bIsDedicated", Session->SessionSettings.bIsDedicated);
	AddLobbyAttribute(LobbyModificationHandle, &Opt7);

	const FLobbyAttributeOptions Opt8("BuildUniqueId", Session->SessionSettings.BuildUniqueId);
	AddLobbyAttribute(LobbyModificationHandle, &Opt8);

	// Add all of the custom settings
	for (FSessionSettings::TConstIterator It(Session->SessionSettings.Settings); It; ++It)
	{
		const FName KeyName = It.Key();
		const FOnlineSessionSetting& Setting = It.Value();

		// Skip unsupported types or non session advertised settings
		if (Setting.AdvertisementType < EOnlineDataAdvertisementType::ViaOnlineService || !IsSessionSettingTypeSupported(Setting.Data.GetType()))
		{
			continue;
		}

		const FLobbyAttributeOptions Attribute(TCHAR_TO_UTF8(*KeyName.ToString()), Setting.Data);
		AddLobbyAttribute(LobbyModificationHandle, &Attribute);
	}

	SetLobbyMemberAttributes(LobbyModificationHandle, EOSSubsystem->UserManager->GetUniquePlayerId(EOSSubsystem->UserManager->GetDefaultLocalUser()).ToSharedRef(), *Session);
}

void FOnlineSessionEOS::SetLobbyMemberAttributes(EOS_HLobbyModification LobbyModificationHandle, FUniqueNetIdRef LobbyMemberId, FNamedOnlineSession& Session)
{
	if (FSessionSettings* MemberSettings = Session.SessionSettings.MemberSettings.Find(LobbyMemberId))
	{
		for (FSessionSettings::TConstIterator It(*MemberSettings); It; ++It)
		{
			const FName KeyName = It.Key();
			const FOnlineSessionSetting& Setting = It.Value();

			// Skip unsupported types or non session advertised settings
			if (Setting.AdvertisementType < EOnlineDataAdvertisementType::ViaOnlineService || !IsSessionSettingTypeSupported(Setting.Data.GetType()))
			{
				continue;
			}

			const FLobbyAttributeOptions Attribute(TCHAR_TO_UTF8(*KeyName.ToString()), Setting.Data);
			AddLobbyMemberAttribute(LobbyModificationHandle, &Attribute);
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionEOS::SetLobbyMemberAttributes] Lobby Member with UniqueNetId [%s] not found in Lobby Session with SessionId [%s]. This is expected when the lobby owner creates the lobby, or when a new member joins."), *LobbyMemberId->ToDebugString(), *Session.GetSessionIdStr());
	}
}

uint32 FOnlineSessionEOS::UpdateLobbySession(FNamedOnlineSession* Session, const FOnUpdateSessionCompleteDelegate& CompletionDelegate)
{
	check(Session != nullptr);

	uint32 Result = ONLINE_FAIL;

	if (Session->SessionState == EOnlineSessionState::Creating)
	{
		Result = ONLINE_IO_PENDING;

		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::UpdateLobbySession] UpdateLobby not successful. Session %s is still being created."), *Session->SessionName.ToString());

		EOSSubsystem->ExecuteNextTick([CompletionDelegate, SessionName = Session->SessionName]()
			{
				CompletionDelegate.ExecuteIfBound(SessionName, false);
			});
	}
	else
	{
		EOS_Lobby_UpdateLobbyModificationOptions UpdateLobbyModificationOptions = { 0 };
		UpdateLobbyModificationOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_UPDATELOBBYMODIFICATION_API_LATEST, 1);
		const FTCHARToUTF8 Utf8LobbyId(*Session->SessionInfo->GetSessionId().ToString());
		UpdateLobbyModificationOptions.LobbyId = (EOS_LobbyId)Utf8LobbyId.Get();
		UpdateLobbyModificationOptions.LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(EOSSubsystem->UserManager->GetDefaultLocalUser()); // Maybe not split screen friendly

		EOS_HLobbyModification LobbyModificationHandle;

		EOS_EResult LobbyModificationResult = EOS_Lobby_UpdateLobbyModification(LobbyHandle, &UpdateLobbyModificationOptions, &LobbyModificationHandle);
		if (LobbyModificationResult == EOS_EResult::EOS_Success)
		{
			// If the user initiating the update is the owner, we will update both lobby settings and member settings
			if (FUniqueNetIdEOS::Cast(*Session->OwningUserId).GetProductUserId() == UpdateLobbyModificationOptions.LocalUserId)
			{
				SetLobbyPermissionLevel(LobbyModificationHandle, Session);
				SetLobbyMaxMembers(LobbyModificationHandle, Session);
				SetLobbyAttributes(LobbyModificationHandle, Session);
			}
			else // In any other case, only member settings will be updated, as per API restrictions
			{
				SetLobbyMemberAttributes(LobbyModificationHandle, EOSSubsystem->UserManager->GetUniquePlayerId(EOSSubsystem->UserManager->GetDefaultLocalUser()).ToSharedRef(), *Session);
			}

			EOS_Lobby_UpdateLobbyOptions UpdateLobbyOptions = { 0 };
			UpdateLobbyOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_UPDATELOBBY_API_LATEST, 1);
			UpdateLobbyOptions.LobbyModificationHandle = LobbyModificationHandle;

			FName SessionName = Session->SessionName;
			FLobbyUpdatedCallback* CallbackObj = new FLobbyUpdatedCallback(FOnlineSessionEOSWeakPtr(AsShared()));
			CallbackObj->CallbackLambda = [this, SessionName, CompletionDelegate](const EOS_Lobby_UpdateLobbyCallbackInfo* Data)
			{
				FNamedOnlineSession* Session = GetNamedSession(SessionName);
				if (Session)
				{
					bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success || Data->ResultCode == EOS_EResult::EOS_Sessions_OutOfSync;
					if (!bWasSuccessful)
					{
						UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::UpdateLobbySession] UpdateLobby not successful. Finished with EOS_EResult %s"), *LexToString(Data->ResultCode));
					}

					CompletionDelegate.ExecuteIfBound(SessionName, bWasSuccessful);
				}
				else
				{
					UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::UpdateLobbySession] Unable to find session %s"), *SessionName.ToString());
					CompletionDelegate.ExecuteIfBound(SessionName, false);
				}
			};

			EOS_Lobby_UpdateLobby(LobbyHandle, &UpdateLobbyOptions, CallbackObj, CallbackObj->GetCallbackPtr());

			EOS_LobbyModification_Release(LobbyModificationHandle);

			Result = ONLINE_IO_PENDING;
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::UpdateLobbySession] UpdateLobbyModification not successful. Finished with EOS_EResult %s"), *LexToString(LobbyModificationResult));

			EOSSubsystem->ExecuteNextTick([CompletionDelegate, SessionName = Session->SessionName]()
				{
					CompletionDelegate.ExecuteIfBound(SessionName, false);
				});
		}
	}

	return Result;
}

uint32 FOnlineSessionEOS::EndLobbySession(FNamedOnlineSession* Session)
{
	// Only called from EndSession/DestroySession and presumes only in InProgress state
	check(Session && Session->SessionState == EOnlineSessionState::InProgress);

	EOSSubsystem->ExecuteNextTick([this, SessionName = Session->SessionName]()
	{
		if (FNamedOnlineSession* Session = GetNamedSession(SessionName))
		{
			Session->SessionState = EOnlineSessionState::Ended;			
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("Session [%s] not found"), *SessionName.ToString());
		}

		TriggerOnEndSessionCompleteDelegates(SessionName, true);
	});

	return ONLINE_IO_PENDING;
}

uint32 FOnlineSessionEOS::DestroyLobbySession(int32 LocalUserNum, FNamedOnlineSession* Session, const FOnDestroySessionCompleteDelegate& CompletionDelegate)
{
	check(Session != nullptr);

	uint32 Result = ONLINE_FAIL;

	if (Session->SessionInfo.IsValid())
	{
		Session->SessionState = EOnlineSessionState::Destroying;

		FOnlineSessionInfoEOS* SessionInfo = (FOnlineSessionInfoEOS*)(Session->SessionInfo.Get());
		check(Session->SessionSettings.bUseLobbiesIfAvailable); // We check if it's a lobby session

		// EOS will use the host migration setting to decide if the lobby is destroyed if it's the owner leaving
		EOS_Lobby_LeaveLobbyOptions LeaveOptions = { 0 };
		LeaveOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_LEAVELOBBY_API_LATEST, 1);
		const FTCHARToUTF8 Utf8LobbyId(*SessionInfo->GetSessionId().ToString());
		LeaveOptions.LobbyId = (EOS_LobbyId)Utf8LobbyId.Get();
		LeaveOptions.LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(LocalUserNum);

		FName SessionName = Session->SessionName;
		FLobbyLeftCallback* LeaveCallbackObj = new FLobbyLeftCallback(FOnlineSessionEOSWeakPtr(AsShared()));
		LobbyLeftCallback = LeaveCallbackObj;
		LeaveCallbackObj->CallbackLambda = [this, LocalUserNum, SessionName, CompletionDelegate](const EOS_Lobby_LeaveLobbyCallbackInfo* Data)
		{
			FNamedOnlineSession* LobbySession = GetNamedSession(SessionName);
			if (LobbySession)
			{
				bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
				if (bWasSuccessful)
				{
					UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionEOS::DestroyLobbySession] LeaveLobby was successful. LobbyId is %s."), UTF8_TO_TCHAR(Data->LobbyId));
				}
				else
				{
					UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::DestroyLobbySession] LeaveLobby not successful. Finished with EOS_EResult %s"), *LexToString(Data->ResultCode));
				}

#if WITH_EOSVOICECHAT
				if (FEOSVoiceChatUser* VoiceChatUser = static_cast<FEOSVoiceChatUser*>(EOSSubsystem->GetEOSVoiceChatUserInterface(*EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(LocalUserNum))))
				{
					VoiceChatUser->RemoveLobbyRoom(UTF8_TO_TCHAR(Data->LobbyId));
				}
#endif // WITH_EOSVOICECHAT

				EndSessionAnalytics();

				LobbySession->SessionState = EOnlineSessionState::NoSession;

				RemoveNamedSession(SessionName);

				CompletionDelegate.ExecuteIfBound(SessionName, bWasSuccessful);
				TriggerOnDestroySessionCompleteDelegates(SessionName, bWasSuccessful);
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::DestroyLobbySession] Unable to find session %s"), *SessionName.ToString());
				CompletionDelegate.ExecuteIfBound(SessionName, false);
				TriggerOnDestroySessionCompleteDelegates(SessionName, false);
			}
		};

		EOS_Lobby_LeaveLobby(LobbyHandle, &LeaveOptions, LeaveCallbackObj, LeaveCallbackObj->GetCallbackPtr());

		Result = ONLINE_IO_PENDING;
	}

	return Result;
}

uint32 FOnlineSessionEOS::FindLobbySession(int32 SearchingPlayerNum, const TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	uint32 Result = ONLINE_FAIL;

	EOS_Lobby_CreateLobbySearchOptions CreateLobbySearchOptions = { 0 };
	CreateLobbySearchOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_CREATELOBBYSEARCH_API_LATEST, 1);
	CreateLobbySearchOptions.MaxResults = FMath::Clamp(SearchSettings->MaxSearchResults, 0, EOS_LOBBY_MAX_SEARCH_RESULTS);

	EOS_HLobbySearch LobbySearchHandle;

	EOS_EResult SearchResult = EOS_Lobby_CreateLobbySearch(LobbyHandle, &CreateLobbySearchOptions, &LobbySearchHandle);
	if (SearchResult == EOS_EResult::EOS_Success)
	{
		const auto BucketIdUtf8 = StringCast<UTF8CHAR>(*GetBucketId(*SearchSettings));
		FLobbyAttributeOptions BucketIdAttribute(EOS_LOBBY_SEARCH_BUCKET_ID, (const char*)BucketIdUtf8.Get());
		AddLobbySearchAttribute(LobbySearchHandle, &BucketIdAttribute, EOS_EOnlineComparisonOp::EOS_CO_EQUAL);

		// We add the search parameters
		for (FSearchParams::TConstIterator It(SearchSettings->QuerySettings.SearchParams); It; ++It)
		{
			const FName Key = It.Key();
			const FOnlineSessionSearchParam& SearchParam = It.Value();

			// Game server keys are skipped
			if (Key == SEARCH_DEDICATED_ONLY || Key == SETTING_MAPNAME || Key == SEARCH_EMPTY_SERVERS_ONLY || Key == SEARCH_SECURE_SERVERS_ONLY || Key == SEARCH_LOBBIES)
			{
				continue;
			}

			if (!IsSessionSettingTypeSupported(SearchParam.Data.GetType()))
			{
				continue;
			}

			UE_LOG_ONLINE_SESSION(VeryVerbose, TEXT("[FOnlineSessionEOS::FindLobbySession] Adding lobby search param named (%s), (%s)"), *Key.ToString(), *SearchParam.ToString());

			FString ParamName(Key.ToString());
			FLobbyAttributeOptions Attribute(TCHAR_TO_UTF8(*ParamName), SearchParam.Data);
			AddLobbySearchAttribute(LobbySearchHandle, &Attribute, ToEOSSearchOp(SearchParam.ComparisonOp));
		}

		StartLobbySearch(SearchingPlayerNum, LobbySearchHandle, SearchSettings, FOnSingleSessionResultCompleteDelegate::CreateLambda([this](int32 LocalUserNum, bool bWasSuccessful, const FOnlineSessionSearchResult& EOSResult)
		{
			TriggerOnFindSessionsCompleteDelegates(bWasSuccessful);
		}));

		Result = ONLINE_IO_PENDING;
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::FindLobbySession] CreateLobbySearch not successful. Finished with EOS_EResult %s"), *LexToString(SearchResult));
	}

	return Result;
}

void FOnlineSessionEOS::StartLobbySearch(int32 SearchingPlayerNum, EOS_HLobbySearch LobbySearchHandle, const TSharedRef<FOnlineSessionSearch>& SearchSettings, const FOnSingleSessionResultCompleteDelegate& CompletionDelegate)
{
	SessionSearchStartInSeconds = FPlatformTime::Seconds();

	EOS_LobbySearch_FindOptions FindOptions = { 0 };
	FindOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYSEARCH_FIND_API_LATEST, 1);
	FindOptions.LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(SearchingPlayerNum);

	FLobbySearchFindCallback* CallbackObj = new FLobbySearchFindCallback(FOnlineSessionEOSWeakPtr(AsShared()));
	LobbySearchFindCallback = CallbackObj;
	CallbackObj->CallbackLambda = [this, SearchingPlayerNum, LobbySearchHandle, SearchSettings, CompletionDelegate](const EOS_LobbySearch_FindCallbackInfo* Data)
	{
		if (!CurrentSessionSearch.IsValid())
		{
			UE_LOG_ONLINE_SESSION(Log, TEXT("[FOnlineSessionEOS::StartLobbySearch] Current session search is invalid. It may have been canceled."));
			CompletionDelegate.ExecuteIfBound(SearchingPlayerNum, false, FOnlineSessionSearchResult());
		}
		
		else if (Data->ResultCode == EOS_EResult::EOS_Success)
		{
			UE_LOG_ONLINE_SESSION(Log, TEXT("[FOnlineSessionEOS::StartLobbySearch] LobbySearch_Find was successful."));

			CurrentSessionSearch->SearchState = EOnlineAsyncTaskState::Done;

			EOS_LobbySearch_GetSearchResultCountOptions GetSearchResultCountOptions = { 0 };
			GetSearchResultCountOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYSEARCH_GETSEARCHRESULTCOUNT_API_LATEST, 1);

			uint32_t SearchResultsCount = EOS_LobbySearch_GetSearchResultCount(LobbySearchHandle, &GetSearchResultCountOptions);

			if (SearchResultsCount > 0)
			{
				EOS_LobbySearch_CopySearchResultByIndexOptions CopySearchResultByIndexOptions = { 0 };
				CopySearchResultByIndexOptions.ApiVersion = 1;
				UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYSEARCH_COPYSEARCHRESULTBYINDEX_API_LATEST, 1);

				for (uint32_t LobbyIndex = 0; LobbyIndex < SearchResultsCount; LobbyIndex++)
				{
					EOS_HLobbyDetails LobbyDetailsHandle;

					CopySearchResultByIndexOptions.LobbyIndex = LobbyIndex;

					EOS_EResult Result = EOS_LobbySearch_CopySearchResultByIndex(LobbySearchHandle, &CopySearchResultByIndexOptions, &LobbyDetailsHandle);
					if (Result == EOS_EResult::EOS_Success)
					{
						UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionEOS::StartLobbySearch::FLobbySearchFindCallback] LobbySearch_CopySearchResultByIndex was successful."));
						const TSharedRef<FLobbyDetailsEOS> LobbyDetails = MakeShared<FLobbyDetailsEOS>(LobbyDetailsHandle);
						LobbySearchResultsPendingIdResolution.Add(LobbyDetails);
					}
					else
					{
						// It's unlikely EOS_LobbySearch_CopySearchResultByIndex would return failure. If it does return to the caller the failure and stop copying search results
						UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::StartLobbySearch::FLobbySearchFindCallback] LobbySearch_CopySearchResultByIndex not successful. Finished with EOS_EResult %s"), *LexToString(Data->ResultCode));
						CompletionDelegate.ExecuteIfBound(SearchingPlayerNum, false, SearchSettings->SearchResults.Last());
						return; 
					}
				}

				// Make a copy to iterate over, as the AddLobbySearchResult delegate removes entries.
				const TArray<TSharedRef<FLobbyDetailsEOS>> LobbySearchResultsPendingIdResolutionCopy = LobbySearchResultsPendingIdResolution;
				for (const TSharedRef<FLobbyDetailsEOS>& LobbyDetails : LobbySearchResultsPendingIdResolutionCopy)
				{
					AddLobbySearchResult(LobbyDetails, SearchSettings, [this, LobbyDetails, CompletionDelegate, SearchingPlayerNum, SearchSettings](bool bWasSuccessful)
					{
						LobbySearchResultsPendingIdResolution.Remove(LobbyDetails);

						if (LobbySearchResultsPendingIdResolution.IsEmpty())
						{
							// If we fail to copy the lobby data, we won't add a new search result, so we'll return an empty one
							CompletionDelegate.ExecuteIfBound(SearchingPlayerNum, bWasSuccessful, bWasSuccessful ? SearchSettings->SearchResults.Last() : FOnlineSessionSearchResult());
						}
					});
				}
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Log, TEXT("[FOnlineSessionEOS::StartLobbySearch::FLobbySearchFindCallback] LobbySearch_GetSearchResultCount returned no results"));

				CompletionDelegate.ExecuteIfBound(SearchingPlayerNum, true, FOnlineSessionSearchResult());
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::StartLobbySearch::FLobbySearchFindCallback] LobbySearch_Find not successful. Finished with EOS_EResult %s"), *LexToString(Data->ResultCode));

			CurrentSessionSearch->SearchState = EOnlineAsyncTaskState::Failed;

			CompletionDelegate.ExecuteIfBound(SearchingPlayerNum, false, FOnlineSessionSearchResult());
		}

		EOS_LobbySearch_Release(LobbySearchHandle);
	};

	EOS_LobbySearch_Find(LobbySearchHandle, &FindOptions, CallbackObj, CallbackObj->GetCallbackPtr());
}

void FOnlineSessionEOS::AddLobbySearchResult(const TSharedRef<FLobbyDetailsEOS>& LobbyDetails, const TSharedRef<FOnlineSessionSearch>& SearchSettings, FOnCopyLobbyDataCompleteCallback&& Callback)
{
	EOS_LobbyDetails_Info* LobbyDetailsInfo = nullptr;
	EOS_LobbyDetails_CopyInfoOptions CopyOptions = { };
	CopyOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYDETAILS_COPYINFO_API_LATEST, 1);
	EOS_EResult CopyResult = EOS_LobbyDetails_CopyInfo(LobbyDetails->LobbyDetailsHandle, &CopyOptions, &LobbyDetailsInfo);
	if (CopyResult == EOS_EResult::EOS_Success)
	{
		int32 Position = SearchSettings->SearchResults.AddZeroed();
		FOnlineSessionSearchResult& SearchResult = SearchSettings->SearchResults[Position];
		SearchResult.PingInMs = static_cast<int32>((FPlatformTime::Seconds() - SessionSearchStartInSeconds) * 1000);

		TSharedPtr<FOnlineSessionInfoEOS> SessionInfo = MakeShared<FOnlineSessionInfoEOS>(FOnlineSessionInfoEOS::Create(FUniqueNetIdEOSLobby::Create(UTF8_TO_TCHAR(LobbyDetailsInfo->LobbyId)), LobbyDetails));

		const FName NetDriverName = GetDefault<UNetDriverEOS>()->NetDriverName;
		SessionInfo->HostAddr = MakeShared<FInternetAddrEOS>(LobbyDetailsInfo->LobbyOwnerUserId);

		SearchResult.Session.SessionInfo = SessionInfo;

		// We copy the lobby data and settings, but not the member data (for search results)
		CopyLobbyData(LobbyDetails, LobbyDetailsInfo, SearchResult.Session, false, MoveTemp(Callback));

		EOS_LobbyDetails_Info_Release(LobbyDetailsInfo);

		// We don't release the details handle here, because we'll use it for the join operation
	}
	else
	{
		// CopyLobbyData may launch an asynchronous operation, so we'll delay the execution of this callback to match the flow
		EOSSubsystem->ExecuteNextTick([CopyResult, Callback = MoveTemp(Callback)]()
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionEOS::AddLobbySearchResult] LobbyDetails_CopyInfo not successful. Finished with EOS_EResult %s"), *LexToString(CopyResult));

			Callback(false);
		});
	}
}

void FOnlineSessionEOS::CopyLobbyData(const TSharedRef<FLobbyDetailsEOS>& LobbyDetails, EOS_LobbyDetails_Info* LobbyDetailsInfo, FOnlineSession& OutSession, bool bCopyMemberData, FOnCopyLobbyDataCompleteCallback&& Callback)
{
	// This method launches an asynchronous operation, so we'll pass the details handle as a shared ref to make sure it stays alive

	// bUsesPresence will be set to false by default in search results, and it should be set by the game side before calling JoinSession.

	OutSession.SessionSettings.bUseLobbiesIfAvailable = true;
	OutSession.SessionSettings.bIsLANMatch = false;
	OutSession.SessionSettings.Set(SETTING_HOST_MIGRATION, LobbyDetailsInfo->bAllowHostMigration, EOnlineDataAdvertisementType::DontAdvertise);
#if WITH_EOS_RTC
	OutSession.SessionSettings.bUseLobbiesVoiceChatIfAvailable = LobbyDetailsInfo->bRTCRoomEnabled == EOS_TRUE;
#endif

	switch (LobbyDetailsInfo->PermissionLevel)
	{
	case EOS_ELobbyPermissionLevel::EOS_LPL_PUBLICADVERTISED:
	case EOS_ELobbyPermissionLevel::EOS_LPL_JOINVIAPRESENCE:
		OutSession.SessionSettings.bAllowJoinViaPresence = true;

		OutSession.SessionSettings.NumPublicConnections = LobbyDetailsInfo->MaxMembers;
		OutSession.NumOpenPublicConnections = LobbyDetailsInfo->AvailableSlots;

		break;
	case EOS_ELobbyPermissionLevel::EOS_LPL_INVITEONLY:
		OutSession.SessionSettings.bAllowJoinViaPresence = false;

		OutSession.SessionSettings.NumPrivateConnections = LobbyDetailsInfo->MaxMembers;
		OutSession.NumOpenPrivateConnections = LobbyDetailsInfo->AvailableSlots;

		break;
	}

	OutSession.SessionSettings.bAllowInvites = (bool)LobbyDetailsInfo->bAllowInvites;

	// We copy the settings related to lobby attributes
	CopyLobbyAttributes(*LobbyDetails, OutSession);

	if (bCopyMemberData)
	{
		// Then we copy the settings for all lobby members
		EOS_LobbyDetails_GetMemberCountOptions CountOptions = { };
		CountOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYDETAILS_GETMEMBERCOUNT_API_LATEST, 1);
		int32 Count = EOS_LobbyDetails_GetMemberCount(LobbyDetails->LobbyDetailsHandle, &CountOptions);

		TArray<EOS_ProductUserId> TargetUserIds;
		TargetUserIds.Reserve(Count);
		for (int32 Index = 0; Index < Count; Index++)
		{
			EOS_LobbyDetails_GetMemberByIndexOptions GetMemberByIndexOptions = { };
			GetMemberByIndexOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYDETAILS_GETMEMBERBYINDEX_API_LATEST, 1);
			GetMemberByIndexOptions.MemberIndex = Index;

			if (EOS_ProductUserId TargetUserId = EOS_LobbyDetails_GetMemberByIndex(LobbyDetails->LobbyDetailsHandle, &GetMemberByIndexOptions))
			{
				TargetUserIds.Add(TargetUserId);
			}
		}

		if (!TargetUserIds.IsEmpty())
		{
			EOSSubsystem->UserManager->ResolveUniqueNetIds(EOSSubsystem->UserManager->GetDefaultLocalUser(), TargetUserIds, [this, LobbyDetails, LobbyId = FUniqueNetIdEOSLobby::Create(UTF8_TO_TCHAR(LobbyDetailsInfo->LobbyId)), Callback = MoveTemp(Callback)](TMap<EOS_ProductUserId, FUniqueNetIdEOSRef> ResolvedUniqueNetIds, const FOnlineError& Error)
				{
					FOnlineSession* Session = GetOnlineSessionFromLobbyId(*LobbyId);
					if (Session)
					{
						// One of the resolved ids will be the Owner's, so we'll set that too
						EOS_LobbyDetails_GetLobbyOwnerOptions GetLobbyOwnerOptions = {};
						GetLobbyOwnerOptions.ApiVersion = 1;
						UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYDETAILS_GETLOBBYOWNER_API_LATEST, 1);

						const EOS_ProductUserId LobbyOwner = EOS_LobbyDetails_GetLobbyOwner(LobbyDetails->LobbyDetailsHandle, &GetLobbyOwnerOptions);

						if (FUniqueNetIdEOSRef* OwnerNetId = ResolvedUniqueNetIds.Find(LobbyOwner))
						{
							Session->OwningUserId = *OwnerNetId;
							Session->OwningUserName = EOSSubsystem->UserManager->GetPlayerNickname(**OwnerNetId);
						}

						for (TMap<EOS_ProductUserId, FUniqueNetIdEOSRef>::TConstIterator It(ResolvedUniqueNetIds); It; ++It)
						{
							FSessionSettings& MemberSettings = Session->SessionSettings.MemberSettings.FindOrAdd(It.Value());

							CopyLobbyMemberAttributes(*LobbyDetails, It.Key(), MemberSettings);
						}

						// We'll update the search result to make sure the data is updated in all copies of the session
						if (FOnlineSessionSearchResult* SearchResult = GetSearchResultFromLobbyId(*LobbyId))
						{
							SearchResult->Session = *Session;
						}
					}

					const bool bWasSuccessful = Session != nullptr;
					Callback(bWasSuccessful);
				});

			return;
		}
	}
	else // If we should not copy the member data, we still need to copy the Owner's Id and Name
	{
		EOS_LobbyDetails_GetLobbyOwnerOptions GetLobbyOwnerOptions = {};
		GetLobbyOwnerOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYDETAILS_GETLOBBYOWNER_API_LATEST, 1);

		const EOS_ProductUserId LobbyOwner = EOS_LobbyDetails_GetLobbyOwner(LobbyDetails->LobbyDetailsHandle, &GetLobbyOwnerOptions);

		EOSSubsystem->UserManager->ResolveUniqueNetIds(EOSSubsystem->UserManager->GetDefaultLocalUser(), { LobbyOwner }, [this, LobbyOwner, LobbyDetails, LobbyId = FUniqueNetIdEOSLobby::Create(UTF8_TO_TCHAR(LobbyDetailsInfo->LobbyId)), Callback = MoveTemp(Callback)](TMap<EOS_ProductUserId, FUniqueNetIdEOSRef> ResolvedUniqueNetIds, const FOnlineError& Error)
			{
				FOnlineSession* Session = GetOnlineSessionFromLobbyId(*LobbyId);
				if (Session)
				{
					FUniqueNetIdEOSRef* OwnerNetId = ResolvedUniqueNetIds.Find(LobbyOwner);
					if (ensure(OwnerNetId))
					{
						Session->OwningUserId = *OwnerNetId;
						Session->OwningUserName = EOSSubsystem->UserManager->GetPlayerNickname(**OwnerNetId);
					}

					// We'll update the search result to make sure the data is updated in all copies of the session
					if (FOnlineSessionSearchResult* SearchResult = GetSearchResultFromLobbyId(*LobbyId))
					{
						SearchResult->Session = *Session;
					}
				}

				const bool bWasSuccessful = Session != nullptr;
				Callback(bWasSuccessful);
			});

		return;
	}

	// ResolveUniqueNetIds is an asynchronous operation, so in the cases where it's not called, we'll delay the execution of this callback to match the flow
	EOSSubsystem->ExecuteNextTick([Callback = MoveTemp(Callback)]()
	{
		Callback(true);
	});
}

void FOnlineSessionEOS::CopyLobbyAttributes(const FLobbyDetailsEOS& LobbyDetails, FOnlineSession& OutSession)
{
	// In this method we are updating/adding attributes, but not removing

	EOS_LobbyDetails_GetAttributeCountOptions CountOptions = { };
	CountOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYDETAILS_GETATTRIBUTECOUNT_API_LATEST, 1);
	int32 Count = EOS_LobbyDetails_GetAttributeCount(LobbyDetails.LobbyDetailsHandle, &CountOptions);

	for (int32 Index = 0; Index < Count; Index++)
	{
		EOS_LobbyDetails_CopyAttributeByIndexOptions AttrOptions = { };
		AttrOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYDETAILS_COPYATTRIBUTEBYINDEX_API_LATEST, 1);
		AttrOptions.AttrIndex = Index;

		EOS_Lobby_Attribute* Attribute = NULL;
		EOS_EResult ResultCode = EOS_LobbyDetails_CopyAttributeByIndex(LobbyDetails.LobbyDetailsHandle, &AttrOptions, &Attribute);
		if (ResultCode == EOS_EResult::EOS_Success)
		{
			FString Key = UTF8_TO_TCHAR(Attribute->Data->Key);
			if (Key == TEXT("NumPublicConnections"))
			{
				OutSession.SessionSettings.NumPublicConnections = Attribute->Data->Value.AsInt64;
			}
			else if (Key == TEXT("NumPrivateConnections"))
			{
				OutSession.SessionSettings.NumPrivateConnections = Attribute->Data->Value.AsInt64;
			}
			else if (Key == TEXT("bAntiCheatProtected"))
			{
				OutSession.SessionSettings.bAntiCheatProtected = Attribute->Data->Value.AsBool == EOS_TRUE;
			}
			else if (Key == TEXT("bUsesStats"))
			{
				OutSession.SessionSettings.bUsesStats = Attribute->Data->Value.AsBool == EOS_TRUE;
			}
			else if (Key == TEXT("bIsDedicated"))
			{
				OutSession.SessionSettings.bIsDedicated = Attribute->Data->Value.AsBool == EOS_TRUE;
			}
			else if (Key == TEXT("BuildUniqueId"))
			{
				OutSession.SessionSettings.BuildUniqueId = Attribute->Data->Value.AsInt64;
			}
			// Handle FSessionSettings
			else
			{
				FOnlineSessionSetting Setting;
				switch (Attribute->Data->ValueType)
				{
				case EOS_ESessionAttributeType::EOS_SAT_Boolean:
				{
					Setting.Data.SetValue(Attribute->Data->Value.AsBool == EOS_TRUE);
					break;
				}
				case EOS_ESessionAttributeType::EOS_SAT_Int64:
				{
					Setting.Data.SetValue(int64(Attribute->Data->Value.AsInt64));
					break;
				}
				case EOS_ESessionAttributeType::EOS_SAT_Double:
				{
					Setting.Data.SetValue(Attribute->Data->Value.AsDouble);
					break;
				}
				case EOS_ESessionAttributeType::EOS_SAT_String:
				{
					Setting.Data.SetValue(UTF8_TO_TCHAR(Attribute->Data->Value.AsUtf8));
					break;
				}
				}

				OutSession.SessionSettings.Settings.Emplace(FName(Key), MoveTemp(Setting));
			}
		}

		EOS_Lobby_Attribute_Release(Attribute);
	}
}

void FOnlineSessionEOS::CopyLobbyMemberAttributes(const FLobbyDetailsEOS& LobbyDetails, const EOS_ProductUserId& TargetUserId, FSessionSettings& OutSessionSettings)
{
	// In this method we are updating/adding attributes, but not removing

	EOS_LobbyDetails_GetMemberAttributeCountOptions GetMemberAttributeCountOptions = {};
	GetMemberAttributeCountOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYDETAILS_GETMEMBERATTRIBUTECOUNT_API_LATEST, 1);
	GetMemberAttributeCountOptions.TargetUserId = TargetUserId;

	uint32_t MemberAttributeCount = EOS_LobbyDetails_GetMemberAttributeCount(LobbyDetails.LobbyDetailsHandle, &GetMemberAttributeCountOptions);
	for (uint32_t MemberAttributeIndex = 0; MemberAttributeIndex < MemberAttributeCount; MemberAttributeIndex++)
	{
		EOS_LobbyDetails_CopyMemberAttributeByIndexOptions AttrOptions = { };
		AttrOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYDETAILS_COPYMEMBERATTRIBUTEBYINDEX_API_LATEST, 1);
		AttrOptions.AttrIndex = MemberAttributeIndex;
		AttrOptions.TargetUserId = TargetUserId;

		EOS_Lobby_Attribute* Attribute = NULL;
		EOS_EResult ResultCode = EOS_LobbyDetails_CopyMemberAttributeByIndex(LobbyDetails.LobbyDetailsHandle, &AttrOptions, &Attribute);
		if (ResultCode == EOS_EResult::EOS_Success)
		{
			FString Key = Attribute->Data->Key;

			FOnlineSessionSetting Setting;
			switch (Attribute->Data->ValueType)
			{
			case EOS_ESessionAttributeType::EOS_SAT_Boolean:
			{
				Setting.Data.SetValue(Attribute->Data->Value.AsBool == EOS_TRUE);
				break;
			}
			case EOS_ESessionAttributeType::EOS_SAT_Int64:
			{
				Setting.Data.SetValue(int64(Attribute->Data->Value.AsInt64));
				break;
			}
			case EOS_ESessionAttributeType::EOS_SAT_Double:
			{
				Setting.Data.SetValue(Attribute->Data->Value.AsDouble);
				break;
			}
			case EOS_ESessionAttributeType::EOS_SAT_String:
			{
				Setting.Data.SetValue(UTF8_TO_TCHAR(Attribute->Data->Value.AsUtf8));
				break;
			}
			}

			if (OutSessionSettings.Contains(FName(Key)))
			{
				OutSessionSettings[FName(Key)] = Setting;
			}
			else
			{
				OutSessionSettings.Add(FName(Key), Setting);
			}
		}
	}
}

bool FOnlineSessionEOS::HandleSessionExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
#if !UE_BUILD_SHIPPING
	bool bWasHandled = true;

	if (FParse::Command(&Cmd, TEXT("FindFriendSession"))) /* ONLINE SESSION FindFriendSession LocalUserNum=0 FriendListName=default FriendIndex=0 */
	{
		int LocalUserNum = 0;
		FParse::Value(Cmd, TEXT("LocalUserNum="), LocalUserNum);

		FString FriendListName;
		FParse::Value(Cmd, TEXT("FriendListName="), FriendListName);

		int FriendIndex = 0;
		FParse::Value(Cmd, TEXT("FriendIndex="), FriendIndex);
		
		TArray<TSharedRef<FOnlineFriend>> FriendList;
		EOSSubsystem->UserManager->GetFriendsList(LocalUserNum, FriendListName, FriendList);
		if (FriendList.Num() > FriendIndex)
		{
			const TSharedRef<FOnlineFriend>& Friend = FriendList[FriendIndex];
			FindFriendSession(LocalUserNum, *Friend->GetUserId());
		}
	}
	else
	{
		bWasHandled = false;
	}

	return bWasHandled;
#else
	return false;
#endif // !UE_BUILD_SHIPPING
}

#endif // WITH_EOS_SDK
