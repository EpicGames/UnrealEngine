// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/ScopeLock.h"
#include "OnlineSessionSettings.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "Online/LANBeacon.h"
#include "OnlineSubsystemEOSTypes.h"

class FOnlineSubsystemEOS;

#if WITH_EOS_SDK
	#include "eos_sessions_types.h"
	#include "eos_lobby_types.h"

static FName EOS_SESSION_ID = TEXT("EOS_SESSION_ID");
static FName EOS_LOBBY_ID = TEXT("EOS_LOBBY_ID");
TEMP_UNIQUENETIDSTRING_SUBCLASS(FUniqueNetIdEOSSession, EOS_SESSION_ID);
TEMP_UNIQUENETIDSTRING_SUBCLASS(FUniqueNetIdEOSLobby, EOS_LOBBY_ID);

struct FSessionSearchEOS
{
	EOS_HSessionSearch SearchHandle;

	FSessionSearchEOS(EOS_HSessionSearch InSearchHandle)
		: SearchHandle(InSearchHandle)
	{
	}

	virtual ~FSessionSearchEOS()
	{
		EOS_SessionSearch_Release(SearchHandle);
	}
};

/**
 * Interface for interacting with EOS sessions
 */
class FOnlineSessionEOS
	: public IOnlineSession
	, public TSharedFromThis<FOnlineSessionEOS, ESPMode::ThreadSafe>
{
public:
	FOnlineSessionEOS() = delete;
	virtual ~FOnlineSessionEOS();

	virtual FUniqueNetIdPtr CreateSessionIdFromString(const FString& SessionIdStr) override;

	FNamedOnlineSession* GetNamedSession(FName SessionName) override
	{
		FScopeLock ScopeLock(&SessionLock);
		for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
		{
			if (Sessions[SearchIndex].SessionName == SessionName)
			{
				return &Sessions[SearchIndex];
			}
		}
		return nullptr;
	}

	virtual void RemoveNamedSession(FName SessionName) override
	{
		FScopeLock ScopeLock(&SessionLock);
		for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
		{
			if (Sessions[SearchIndex].SessionName == SessionName)
			{
				Sessions.RemoveAtSwap(SearchIndex);
				return;
			}
		}
	}

	virtual EOnlineSessionState::Type GetSessionState(FName SessionName) const override
	{
		FScopeLock ScopeLock(&SessionLock);
		for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
		{
			if (Sessions[SearchIndex].SessionName == SessionName)
			{
				return Sessions[SearchIndex].SessionState;
			}
		}

		return EOnlineSessionState::NoSession;
	}

	virtual bool HasPresenceSession() override
	{
		FScopeLock ScopeLock(&SessionLock);
		for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
		{
			if (Sessions[SearchIndex].SessionSettings.bUsesPresence)
			{
				return true;
			}
		}

		return false;
	}

// IOnlineSession Interface

	/** The BucketId for the session can be specified via adding a custom session setting with the key OSSEOS_BUCKET_ID_ATTRIBUTE_KEY. A default value will be used otherwise */
	virtual bool CreateSession(int32 HostingPlayerNum, FName SessionName, const FOnlineSessionSettings& NewSessionSettings) override;
	/** The BucketId for the session can be specified via adding a custom session setting with the key OSSEOS_BUCKET_ID_ATTRIBUTE_KEY. A default value will be used otherwise */
	virtual bool CreateSession(const FUniqueNetId& HostingPlayerId, FName SessionName, const FOnlineSessionSettings& NewSessionSettings) override;
	virtual bool StartSession(FName SessionName) override;
	virtual bool UpdateSession(FName SessionName, FOnlineSessionSettings& UpdatedSessionSettings, bool bShouldRefreshOnlineData = true) override;
	virtual bool EndSession(FName SessionName) override;
	virtual bool DestroySession(FName SessionName, const FOnDestroySessionCompleteDelegate& CompletionDelegate = FOnDestroySessionCompleteDelegate()) override;
	virtual bool IsPlayerInSession(FName SessionName, const FUniqueNetId& UniqueId) override;
	virtual bool StartMatchmaking(const TArray< FUniqueNetIdRef >& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings, TSharedRef<FOnlineSessionSearch>& SearchSettings) override;
	virtual bool CancelMatchmaking(int32 SearchingPlayerNum, FName SessionName) override;
	virtual bool CancelMatchmaking(const FUniqueNetId& SearchingPlayerId, FName SessionName) override;
	/** The BucketId to be used in the search can be specified via adding a custom search filter with the key OSSEOS_BUCKET_ID_ATTRIBUTE_KEY. A default value will be used otherwise */
	virtual bool FindSessions(int32 SearchingPlayerNum, const TSharedRef<FOnlineSessionSearch>& SearchSettings) override;
	/** The BucketId to be used in the search can be specified via adding a custom search filter with the key OSSEOS_BUCKET_ID_ATTRIBUTE_KEY. A default value will be used otherwise */
	virtual bool FindSessions(const FUniqueNetId& SearchingPlayerId, const TSharedRef<FOnlineSessionSearch>& SearchSettings) override;
	virtual bool FindSessionById(const FUniqueNetId& SearchingUserId, const FUniqueNetId& SessionId, const FUniqueNetId& FriendId, const FOnSingleSessionResultCompleteDelegate& CompletionDelegate) override;
	virtual bool CancelFindSessions() override;
	virtual bool PingSearchResults(const FOnlineSessionSearchResult& SearchResult) override;
	virtual bool JoinSession(int32 PlayerNum, FName SessionName, const FOnlineSessionSearchResult& DesiredSession) override;
	virtual bool JoinSession(const FUniqueNetId& PlayerId, FName SessionName, const FOnlineSessionSearchResult& DesiredSession) override;
	virtual bool FindFriendSession(int32 LocalUserNum, const FUniqueNetId& Friend) override;
	virtual bool FindFriendSession(const FUniqueNetId& LocalUserId, const FUniqueNetId& Friend) override;
	virtual bool FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList) override;
	virtual bool SendSessionInviteToFriend(int32 LocalUserNum, FName SessionName, const FUniqueNetId& Friend) override;
	virtual bool SendSessionInviteToFriend(const FUniqueNetId& LocalUserId, FName SessionName, const FUniqueNetId& Friend) override;
	virtual bool SendSessionInviteToFriends(int32 LocalUserNum, FName SessionName, const TArray< FUniqueNetIdRef >& Friends) override;
	virtual bool SendSessionInviteToFriends(const FUniqueNetId& LocalUserId, FName SessionName, const TArray< FUniqueNetIdRef >& Friends) override;
	virtual bool GetResolvedConnectString(FName SessionName, FString& ConnectInfo, FName PortType) override;
	virtual bool GetResolvedConnectString(const FOnlineSessionSearchResult& SearchResult, FName PortType, FString& ConnectInfo) override;
	virtual FOnlineSessionSettings* GetSessionSettings(FName SessionName) override;
	virtual FString GetVoiceChatRoomName(int32 LocalUserNum, const FName& SessionName) override;
	virtual bool RegisterPlayer(FName SessionName, const FUniqueNetId& PlayerId, bool bWasInvited) override;
	virtual bool RegisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasInvited = false) override;
	virtual bool UnregisterPlayer(FName SessionName, const FUniqueNetId& PlayerId) override;
	virtual bool UnregisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players) override;
	virtual void RegisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnRegisterLocalPlayerCompleteDelegate& Delegate) override;
	virtual void UnregisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnUnregisterLocalPlayerCompleteDelegate& Delegate) override;
	virtual void RemovePlayerFromSession(int32 LocalUserNum, FName SessionName, const FUniqueNetId& TargetPlayerId) override;
	virtual int32 GetNumSessions() override;
	virtual void DumpSessionState() override;
// ~IOnlineSession Interface

	/** Critical sections for thread safe operation of session lists */
	mutable FCriticalSection SessionLock;

	/** Current session settings */
	TArray<FNamedOnlineSession> Sessions;

	/** Current search object */
	TSharedPtr<FOnlineSessionSearch> CurrentSessionSearch;

	/** Current search start time. */
	double SessionSearchStartInSeconds;

	FOnlineSessionEOS(FOnlineSubsystemEOS* InSubsystem)
		: CurrentSessionSearch(nullptr)
		, SessionSearchStartInSeconds(0)
		, EOSSubsystem(InSubsystem)
	{
	}

	/**
	 * Session tick for various background tasks
	 */
	void Tick(float DeltaTime);

	// IOnlineSession
	class FNamedOnlineSession* AddNamedSession(FName SessionName, const FOnlineSessionSettings& SessionSettings) override
	{
		FScopeLock ScopeLock(&SessionLock);
		return new (Sessions) FNamedOnlineSession(SessionName, SessionSettings);
	}

	class FNamedOnlineSession* AddNamedSession(FName SessionName, const FOnlineSession& Session) override
	{
		FScopeLock ScopeLock(&SessionLock);
		return new (Sessions) FNamedOnlineSession(SessionName, Session);
	}

	void CheckPendingSessionInvite();

	void UpdateOrAddLobbyMember(const FUniqueNetIdEOSLobbyRef& LobbyNetId, const FUniqueNetIdEOSRef& PlayerId);
	bool AddOnlineSessionMember(FName SessionName, const FUniqueNetIdRef& PlayerId);
	bool RemoveOnlineSessionMember(FName SessionName, const FUniqueNetIdRef& PlayerId);

	void RegisterLocalPlayers(class FNamedOnlineSession* Session);

	void Init();

	bool HandleSessionExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);

private:
	// EOS Lobbies

	EOS_HLobby LobbyHandle;
	TArray<TSharedRef<FLobbyDetailsEOS>> LobbySearchResultsPendingIdResolution;

	// Lobby session callbacks and methods
	FCallbackBase* LobbyCreatedCallback;
	FCallbackBase* LobbySearchFindCallback;
	FCallbackBase* LobbyJoinedCallback;
	FCallbackBase* LobbyLeftCallback;
	FCallbackBase* LobbyDestroyedCallback;
	FCallbackBase* LobbySendInviteCallback;

	uint32 CreateLobbySession(int32 HostingPlayerNum, FNamedOnlineSession* Session);
	void OnCreateLobbySessionUpdateComplete(FName SessionName, bool bWasSuccessful, int32 HostingPlayerNum);
	void DestroyLobbySessionOnCreationUpdateError(int32 LocalUserNum, FNamedOnlineSession* Session);
	uint32 FindLobbySession(int32 SearchingPlayerNum, const TSharedRef<FOnlineSessionSearch>& SearchSettings);
	void StartLobbySearch(int32 SearchingPlayerNum, EOS_HLobbySearch LobbySearchHandle, const TSharedRef<FOnlineSessionSearch>& SearchSettings, const FOnSingleSessionResultCompleteDelegate& CompletionDelegate);
	uint32 JoinLobbySession(int32 PlayerNum, FNamedOnlineSession* Session, const FOnlineSession* SearchSession);
	uint32 UpdateLobbySession(FNamedOnlineSession* Session, const FOnUpdateSessionCompleteDelegate& CompletionDelegate);
	uint32 StartLobbySession(FNamedOnlineSession* Session);
	uint32 EndLobbySession(FNamedOnlineSession* Session);
	uint32 DestroyLobbySession(int32 LocalUserNum, FNamedOnlineSession* Session, const FOnDestroySessionCompleteDelegate& CompletionDelegate);
	bool SendLobbyInvite(FName SessionName, EOS_ProductUserId SenderId, EOS_ProductUserId ReceiverId);

	// Lobby notification callbacks and methods
	EOS_NotificationId LobbyUpdateReceivedId;
	FCallbackBase* LobbyUpdateReceivedCallback;
	EOS_NotificationId LobbyMemberUpdateReceivedId;
	FCallbackBase* LobbyMemberUpdateReceivedCallback;
	EOS_NotificationId LobbyMemberStatusReceivedId;
	FCallbackBase* LobbyMemberStatusReceivedCallback;
	EOS_NotificationId LobbyInviteReceivedId;
	FCallbackBase* LobbyInviteReceivedCallback;
	EOS_NotificationId LobbyInviteAcceptedId;
	FCallbackBase* LobbyInviteAcceptedCallback;
	EOS_NotificationId JoinLobbyAcceptedId;
	FCallbackBase* JoinLobbyAcceptedCallback;
	EOS_NotificationId LeaveLobbyRequestedId;
	FCallbackBase* LeaveLobbyRequestedCallback;

	void OnLobbyUpdateReceived(const EOS_LobbyId& LobbyId);
	void OnLobbyMemberUpdateReceived(const EOS_LobbyId& LobbyId, const EOS_ProductUserId& TargetUserId);
	void OnMemberStatusReceived(const EOS_LobbyId& LobbyId, const EOS_ProductUserId& TargetUserId, EOS_ELobbyMemberStatus CurrentStatus);
	void OnLobbyInviteReceived(const EOS_Lobby_LobbyInviteReceivedCallbackInfo* Data);
	void OnLobbyInviteAccepted(const EOS_Lobby_LobbyInviteAcceptedCallbackInfo* Data);
	void OnJoinLobbyAccepted(const EOS_Lobby_JoinLobbyAcceptedCallbackInfo* Data);
	void OnLeaveLobbyRequested(const EOS_Lobby_LeaveLobbyRequestedCallbackInfo* Data);

	// Methods to update an API Lobby from an OSS Lobby
	void SetLobbyPermissionLevel(EOS_HLobbyModification LobbyModificationHandle, FNamedOnlineSession* Session);
	void SetLobbyMaxMembers(EOS_HLobbyModification LobbyModificationHandle, FNamedOnlineSession* Session);
	void SetLobbyAttributes(EOS_HLobbyModification LobbyModificationHandle, FNamedOnlineSession* Session);
	void AddLobbyAttribute(EOS_HLobbyModification LobbyModificationHandle, const EOS_Lobby_AttributeData* Attribute);
	void SetLobbyMemberAttributes(EOS_HLobbyModification LobbyModificationHandle, FUniqueNetIdRef LobbyMemberId, FNamedOnlineSession& Session);
	void AddLobbyMemberAttribute(EOS_HLobbyModification LobbyModificationHandle, const EOS_Lobby_AttributeData* Attribute);

	// Methods to update an OSS Lobby from an API Lobby
	typedef TFunction<void(bool bWasSuccessful)> FOnCopyLobbyDataCompleteCallback;
	void CopyLobbyData(const TSharedRef<FLobbyDetailsEOS>& LobbyDetails, EOS_LobbyDetails_Info* LobbyDetailsInfo, FOnlineSession& OutSession, bool bCopyMemberData, FOnCopyLobbyDataCompleteCallback&& Callback);
	void CopyLobbyAttributes(const FLobbyDetailsEOS& LobbyDetails, FOnlineSession& OutSession);
	void CopyLobbyMemberAttributes(const FLobbyDetailsEOS& LobbyDetails, const EOS_ProductUserId& TargetUserId, FSessionSettings& OutSessionSettings);

	// Lobby search
	void AddLobbySearchAttribute(EOS_HLobbySearch LobbySearchHandle, const EOS_Lobby_AttributeData* Attribute, EOS_EOnlineComparisonOp ComparisonOp);
	void AddLobbySearchResult(const TSharedRef<FLobbyDetailsEOS>& LobbyDetails, const TSharedRef<FOnlineSessionSearch>& SearchSettings, FOnCopyLobbyDataCompleteCallback&& Callback);

	// Helper methods
	typedef TFunction<void(const EOS_ProductUserId& ProductUserId, EOS_EpicAccountId& EpicAccountId)> GetEpicAccountIdAsyncCallback;

	void RegisterSessionNotifications();
	void OnSessionInviteReceived(const EOS_Sessions_SessionInviteReceivedCallbackInfo* Data);
	void OnSessionInviteAccepted(const EOS_Sessions_SessionInviteAcceptedCallbackInfo* Data);
	void OnJoinSessionAccepted(const EOS_Sessions_JoinSessionAcceptedCallbackInfo* Data);
	void OnLeaveSessionRequested(const EOS_Sessions_LeaveSessionRequestedCallbackInfo* Data);

	void RegisterLobbyNotifications();
	FNamedOnlineSession* GetNamedSessionFromLobbyId(const FUniqueNetIdEOSLobby& LobbyId);
	FOnlineSessionSearchResult* GetSearchResultFromLobbyId(const FUniqueNetIdEOSLobby& LobbyId);
	FOnlineSession* GetOnlineSessionFromLobbyId(const FUniqueNetIdEOSLobby& LobbyId);
	EOS_ELobbyPermissionLevel GetLobbyPermissionLevelFromSessionSettings(const FOnlineSessionSettings& SessionSettings);
	uint32_t GetLobbyMaxMembersFromSessionSettings(const FOnlineSessionSettings& SessionSettings);
	static FString GetBucketId(const FOnlineSessionSettings& SessionSettings);
	static FString GetBucketId(const FOnlineSessionSearch& SessionSettings);

	// EOS Sessions
	TArray<TSharedRef<FSessionDetailsEOS>> SessionSearchResultsPendingIdResolution;

	uint32 CreateEOSSession(int32 HostingPlayerNum, FNamedOnlineSession* Session);
	uint32 JoinEOSSession(int32 PlayerNum, FNamedOnlineSession* Session);
	uint32 StartEOSSession(FNamedOnlineSession* Session);
	uint32 UpdateEOSSession(FNamedOnlineSession* Session);
	uint32 EndEOSSession(FNamedOnlineSession* Session);
	uint32 DestroyEOSSession(FNamedOnlineSession* Session, const FOnDestroySessionCompleteDelegate& CompletionDelegate);
	uint32 FindEOSSession(int32 SearchingPlayerNum, const TSharedRef<FOnlineSessionSearch>& SearchSettings);
	bool SendEOSSessionInvite(FName SessionName, EOS_ProductUserId SenderId, EOS_ProductUserId ReceiverId);
	void FindEOSSessionById(int32 SearchingPlayerNum, const FUniqueNetId& SessionId, const FOnSingleSessionResultCompleteDelegate& CompletionDelegate);

	bool SendSessionInvite(FName SessionName, EOS_ProductUserId SenderId, EOS_ProductUserId ReceiverId);

	void BeginSessionAnalytics(FNamedOnlineSession* Session);
	void EndSessionAnalytics();

	typedef TFunction<void(bool bWasSuccessful)> FOnCopySessionDataCompleteCallback;
	void AddSearchResult(const TSharedRef<FSessionDetailsEOS>& SessionHandle, const TSharedRef<FOnlineSessionSearch>& SearchSettings, FOnCopySessionDataCompleteCallback&& Callback);
	void AddSearchAttribute(EOS_HSessionSearch SearchHandle, const EOS_Sessions_AttributeData* Attribute, EOS_EOnlineComparisonOp ComparisonOp);
	void CopyAttributes(const FSessionDetailsEOS& SessionHandle, FOnlineSession& OutSession);

	void SetPermissionLevel(EOS_HSessionModification SessionModHandle, FNamedOnlineSession* Session);
	void SetMaxPlayers(EOS_HSessionModification SessionModHandle, FNamedOnlineSession* Session);
	void SetInvitesAllowed(EOS_HSessionModification SessionModHandle, FNamedOnlineSession* Session);
	void SetJoinInProgress(EOS_HSessionModification SessionModHandle, FNamedOnlineSession* Session);
	void AddAttribute(EOS_HSessionModification SessionModHandle, const EOS_Sessions_AttributeData* Attribute);
	void SetAttributes(EOS_HSessionModification SessionModHandle, FNamedOnlineSession* Session);
	typedef TEOSCallback<EOS_Sessions_OnUpdateSessionCallback, EOS_Sessions_UpdateSessionCallbackInfo, FOnlineSessionEOS> FUpdateSessionCallback;
	uint32 SharedSessionUpdate(EOS_HSessionModification SessionModHandle, FNamedOnlineSession* Session, FUpdateSessionCallback* Callback);

	void TickLanTasks(float DeltaTime);
	uint32 CreateLANSession(int32 HostingPlayerNum, FNamedOnlineSession* Session);
	uint32 JoinLANSession(int32 PlayerNum, class FNamedOnlineSession* Session, const class FOnlineSession* SearchSession);
	uint32 FindLANSession();

	void AppendSessionToPacket(class FNboSerializeToBufferEOS& Packet, class FOnlineSession* Session);
	void AppendSessionSettingsToPacket(class FNboSerializeToBufferEOS& Packet, FOnlineSessionSettings* SessionSettings);
	void ReadSessionFromPacket(class FNboSerializeFromBufferEOS& Packet, class FOnlineSession* Session);
	void ReadSettingsFromPacket(class FNboSerializeFromBufferEOS& Packet, FOnlineSessionSettings& SessionSettings);
	void OnValidQueryPacketReceived(uint8* PacketData, int32 PacketLength, uint64 ClientNonce);
	void OnValidResponsePacketReceived(uint8* PacketData, int32 PacketLength);
	void OnLANSearchTimeout();
	static void SetPortFromNetDriver(const FOnlineSubsystemEOS& Subsystem, const TSharedPtr<FOnlineSessionInfo>& SessionInfo);
	bool IsHost(const FNamedOnlineSession& Session) const;

	int32 GetDefaultLocalUserForLobby(const FUniqueNetIdString& SessionId);

	/** Reference to the main EOS subsystem */
	FOnlineSubsystemEOS* EOSSubsystem;

	/** Handles advertising sessions over LAN and client searches */
	TSharedPtr<FLANSession> LANSession;
	/** EOS handle wrapper to hold onto it for scope of the search */
	TSharedPtr<FSessionSearchEOS> CurrentSearchHandle;
	/** The last accepted invite search. It searches by session id */
	TSharedPtr<FOnlineSessionSearch> LastInviteSearch;

	/** Used to track adding asynchronous session search results */
	bool bAggregatedAddSearchResultSuccessful;

	/** Notification state for SDK events */
	EOS_NotificationId SessionInviteReceivedId;
	FCallbackBase* SessionInviteReceivedCallback;
	EOS_NotificationId SessionInviteAcceptedId;
	FCallbackBase* SessionInviteAcceptedCallback;
	EOS_NotificationId JoinSessionAcceptedId;
	FCallbackBase* JoinSessionAcceptedCallback;
	EOS_NotificationId LeaveSessionRequestedId;
	FCallbackBase* LeaveSessionRequestedCallback;
};

typedef TSharedPtr<FOnlineSessionEOS, ESPMode::ThreadSafe> FOnlineSessionEOSPtr;
typedef TWeakPtr<FOnlineSessionEOS, ESPMode::ThreadSafe> FOnlineSessionEOSWeakPtr;

#endif
