// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineStatsEOS.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemEOS.h"
#include "OnlineSubsystemEOSPrivate.h"
#include "OnlineSubsystemEOSTypes.h"
#include "UserManagerEOS.h"

#if WITH_EOS_SDK
#include "eos_stats.h"

#define FLOAT_STAT_SCALER 1000.f

inline int32 FloatStatToIntStat(float Value)
{
	return FMath::TruncToInt(Value * FLOAT_STAT_SCALER);
}

inline float IntStatToFloatStat(int32 Value)
{
	return (float)Value / FLOAT_STAT_SCALER;
}

struct FStatNameBuffer
{
	char StatName[EOS_OSS_STRING_BUFFER_LENGTH];
};


void FOnlineStatsEOS::QueryStats(const FUniqueNetIdRef LocalUserId, const FUniqueNetIdRef StatsUser, const FOnlineStatsQueryUserStatsComplete& Delegate)
{
	UE_LOG_ONLINE_STATS(Warning, TEXT("QueryStats() without a list of stats names to query is not supported"));
	const TSharedPtr<const FOnlineStatsUserStats> Result = MakeShared<const FOnlineStatsUserStats>(StatsUser);
	Delegate.ExecuteIfBound(FOnlineError(EOnlineErrorResult::NotImplemented), Result);
}

struct FQueryStatsOptions :
	public EOS_Stats_QueryStatsOptions
{
	TArray<char*> PointerArray;

	FQueryStatsOptions(const uint32 InNumStatIds) :
		EOS_Stats_QueryStatsOptions()
	{
		ApiVersion = 3;
		UE_EOS_CHECK_API_MISMATCH(EOS_STATS_QUERYSTATS_API_LATEST, 3);

		StartTime = EOS_STATS_TIME_UNDEFINED;
		EndTime = EOS_STATS_TIME_UNDEFINED;

		PointerArray.AddZeroed(InNumStatIds);
		for (int32 Index = 0; Index < PointerArray.Num(); Index++)
		{
			PointerArray[Index] = new char[EOS_OSS_STRING_BUFFER_LENGTH];
		}
		StatNames = (const char**)PointerArray.GetData();
		StatNamesCount = InNumStatIds;
	}

	~FQueryStatsOptions()
	{
		for (int32 Index = 0; Index < PointerArray.Num(); Index++)
		{
			delete [] PointerArray[Index];
		}
	}
};

typedef TEOSCallback<EOS_Stats_OnQueryStatsCompleteCallback, EOS_Stats_OnQueryStatsCompleteCallbackInfo, FOnlineStatsEOS> FReadStatsCallback;

struct FStatsQueryContext
{
	int32 NumPlayerReads;
	TArray<FString> StatNames;
	FOnlineStatsQueryUsersStatsComplete Delegate;
	/** Stats are added here as they come in from the service. They are added to the global cache at the end */
	TUniqueNetIdMap<TSharedRef<FOnlineStatsUserStats>> StatsCache;

	FStatsQueryContext(int32 InNumPlayerReads, const TArray<FString>& InStatNames, const FOnlineStatsQueryUsersStatsComplete& InDelegate)
		: NumPlayerReads(InNumPlayerReads)
		, StatNames(InStatNames)
		, Delegate(InDelegate)
	{
	}
};

typedef TSharedPtr<FStatsQueryContext> FStatsQueryContextPtr;

// N^2 append operation
void AppendStats(TUniqueNetIdMap<TSharedRef<FOnlineStatsUserStats>>& StatsCache, const TUniqueNetIdMap<TSharedRef<FOnlineStatsUserStats>>& StatsCacheToAppend)
{
	for (const TPair<FUniqueNetIdRef, TSharedRef<FOnlineStatsUserStats>>& StatsUser : StatsCacheToAppend)
	{
		TSharedRef<FOnlineStatsUserStats>* UserCachedStats = StatsCache.Find(StatsUser.Key);
		if (!UserCachedStats)
		{
			UserCachedStats = &StatsCache.Emplace(StatsUser.Key, MakeShared<FOnlineStatsUserStats>(StatsUser.Key));
		}

		for (TPair<FString, FOnlineStatValue>& NewStat : StatsUser.Value->Stats)
		{
			FOnlineStatValue& StatToUpdate = (*UserCachedStats)->Stats.FindOrAdd(NewStat.Key);
			StatToUpdate = NewStat.Value;
		}
	}
}

void FOnlineStatsEOS::QueryStats(const FUniqueNetIdRef LocalUserId, const TArray<FUniqueNetIdRef>& StatUsers, const TArray<FString>& StatNames, const FOnlineStatsQueryUsersStatsComplete& Delegate)
{
	const FUniqueNetIdEOS& LocalEOSId = FUniqueNetIdEOS::Cast(*LocalUserId);
	const EOS_ProductUserId LocalEOSUserId = LocalEOSId.GetProductUserId();
	if (StatNames.Num() == 0)
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("QueryStats() without a list of stats names to query is not supported"));
		Delegate.ExecuteIfBound(FOnlineError(EOnlineErrorResult::NotImplemented), TArray<TSharedRef<const FOnlineStatsUserStats>>());
		return;
	}
	if (StatUsers.Num() == 0)
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("QueryStats() without a list of users to query is not supported"));
		Delegate.ExecuteIfBound(FOnlineError(EOnlineErrorResult::NotImplemented), TArray<TSharedRef<const FOnlineStatsUserStats>>());
		return;
	}

	// Do the stat name building once rather than for each person
	FQueryStatsOptions Options(StatNames.Num());
	for (int32 Index = 0; Index < StatNames.Num(); Index++)
	{
		FCStringAnsi::Strncpy(Options.PointerArray[Index], TCHAR_TO_UTF8(*StatNames[Index].ToUpper()), EOS_OSS_STRING_BUFFER_LENGTH);
	}

	int32 NumKnownUsers = 0;
	// Validate the number of known users
	for (const FUniqueNetIdRef& StatUserId : StatUsers)
	{
		const FUniqueNetIdEOS& EOSId = FUniqueNetIdEOS::Cast(*StatUserId);
		const EOS_ProductUserId UserId = EOSId.GetProductUserId();
		if (UserId == nullptr)
		{
			continue;
		}
		NumKnownUsers++;
	}

	// This object will live across all calls and be freed at the end
	FStatsQueryContextPtr StatsQueryContext = MakeShareable(new FStatsQueryContext(NumKnownUsers, StatNames, Delegate));
	// Kick off a read for each user
	for (const FUniqueNetIdRef& StatUserId : StatUsers)
	{
		const FUniqueNetIdEOS& EOSId = FUniqueNetIdEOS::Cast(*StatUserId);
		const EOS_ProductUserId TargetEOSUserId = EOSId.GetProductUserId();
		if (TargetEOSUserId == nullptr)
		{
			continue;
		}

		Options.LocalUserId = LocalEOSUserId;
		Options.TargetUserId = TargetEOSUserId;

		FReadStatsCallback* CallbackObj = new FReadStatsCallback(FOnlineStatsEOSWeakPtr(AsShared()));
		CallbackObj->CallbackLambda = [this, StatUserId, StatsQueryContext](const EOS_Stats_OnQueryStatsCompleteCallbackInfo* Data)
		{
			StatsQueryContext->NumPlayerReads--;
			bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
			if (bWasSuccessful)
			{
				char StatNameANSI[EOS_OSS_STRING_BUFFER_LENGTH];
				EOS_Stats_CopyStatByNameOptions Options = { };
				Options.ApiVersion = 1;
				UE_EOS_CHECK_API_MISMATCH(EOS_STATS_COPYSTATBYNAME_API_LATEST, 1);
				Options.TargetUserId = Data->TargetUserId;
				Options.Name = StatNameANSI;

				TSharedPtr<FOnlineStatsUserStats> UserStats = StatsCache.Emplace(StatUserId, MakeShared<FOnlineStatsUserStats>(StatUserId));
				// Read each stat that we were looking for so we can mark missing ones as "empty"
				for (const FString& StatName : StatsQueryContext->StatNames)
				{
					FCStringAnsi::Strncpy(StatNameANSI, TCHAR_TO_UTF8(*StatName.ToUpper()), EOS_OSS_STRING_BUFFER_LENGTH);

					EOS_Stats_Stat* ReadStat = nullptr;
					if (EOS_Stats_CopyStatByName(EOSSubsystem->StatsHandle, &Options, &ReadStat) == EOS_EResult::EOS_Success)
					{
						UE_LOG_ONLINE_STATS(VeryVerbose, TEXT("Found value for stat %s"), *StatName);

						UserStats->Stats.Add(StatName, FOnlineStatValue(ReadStat->Value));

						EOS_Stats_Stat_Release(ReadStat);
					}
					else
					{
						// Put an empty stat in
						UE_LOG_ONLINE_STATS(VeryVerbose, TEXT("Value not found for stat %s, adding empty value"), *StatName);
						UserStats->Stats.Add(StatName, FOnlineStatValue());
					}
				}
			}
			else
			{
				UE_LOG_ONLINE_STATS(Error, TEXT("EOS_Stats_QueryStats() for user (%s) failed with EOS result code (%s)"), *LexToString(Data->TargetUserId), *LexToString(Data->ResultCode));
			}
			if (StatsQueryContext->NumPlayerReads <= 0)
			{
				TArray<TSharedRef<const FOnlineStatsUserStats>> OutArray;
				for (const TPair<FUniqueNetIdRef, TSharedRef<FOnlineStatsUserStats>>& StatsUser : StatsCache)
				{
					OutArray.Add(StatsUser.Value);
				}
				StatsQueryContext->Delegate.ExecuteIfBound(FOnlineError(StatsCache.Num() > 0), OutArray);
			}
		};
		EOS_Stats_QueryStats(EOSSubsystem->StatsHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
	}
}

TSharedPtr<const FOnlineStatsUserStats> FOnlineStatsEOS::GetStats(const FUniqueNetIdRef StatsUserId) const
{
	if (const TSharedRef<FOnlineStatsUserStats>* const FoundStats = StatsCache.Find(StatsUserId))
	{
		return *FoundStats;
	}
	return nullptr;
}

inline int32 GetVariantValue(const FOnlineStatValue& Data)
{
	int32 Value = 0;
	switch (Data.GetType())
	{
		case EOnlineKeyValuePairDataType::Int32:
		{
			Data.GetValue(Value);
			break;
		}
		case EOnlineKeyValuePairDataType::UInt32:
		{
			uint32 Value1 = 0;
			Data.GetValue(Value);
			Value = (int32)Value1;
			break;
		}
		case EOnlineKeyValuePairDataType::Int64:
		{
			int64 Value1 = 0;
			Data.GetValue(Value1);
			Value = (int32)Value1;
			break;
		}
		case EOnlineKeyValuePairDataType::UInt64:
		{
			uint64 Value1 = 0;
			Data.GetValue(Value1);
			Value = (int32)Value1;
			break;
		}
		case EOnlineKeyValuePairDataType::Bool:
		{
			bool Value1 = false;
			Data.GetValue(Value1);
			Value = Value1 ? 1 : 0;
			break;
		}
		case EOnlineKeyValuePairDataType::Float:
		{
			float Value1 = 0.f;
			Data.GetValue(Value1);
			Value = FloatStatToIntStat(Value1);
			break;
		}
		case EOnlineKeyValuePairDataType::Double:
		{
			double Value1 = 0.f;
			Data.GetValue(Value1);
			Value = FloatStatToIntStat(Value1);
			break;
		}
	}
	return Value;
}

typedef TEOSCallback<EOS_Stats_OnIngestStatCompleteCallback, EOS_Stats_IngestStatCompleteCallbackInfo, FOnlineStatsEOS> FWriteStatsCallback;

void FOnlineStatsEOS::WriteStats(EOS_ProductUserId LocalUserId, EOS_ProductUserId UserId, const FOnlineStatsUserUpdatedStats& PlayerStats)
{
	TArray<EOS_Stats_IngestData> EOSData;
	TArray<FTCHARToUTF8> EOSStatNamesUtf8;
	// Preallocate all of the memory
	EOSData.AddZeroed(PlayerStats.Stats.Num());
	EOSStatNamesUtf8.Reserve(PlayerStats.Stats.Num());
	uint32 Index = 0;
	// Convert the stats to the EOS format
	for (const TPair<FString, FOnlineStatUpdate>& Stat : PlayerStats.Stats)
	{
		FString StatKeyUppercase = Stat.Key.ToUpper();
		EOSStatNamesUtf8.Emplace(*StatKeyUppercase);

		if (Stat.Key != StatKeyUppercase)
		{
			UE_LOG_ONLINE_STATS(Warning, TEXT("WriteStats() stat name (%s) is not upper case. Ingest will fail if the stat name is not upper case in the Developer Portal."), *Stat.Key);
		}

		EOS_Stats_IngestData& EOSStat = EOSData[Index];
		EOSStat.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_STATS_INGESTDATA_API_LATEST, 1);
		EOSStat.IngestAmount = GetVariantValue(Stat.Value.GetValue());
		EOSStat.StatName = EOSStatNamesUtf8.Last().Get();

		Index++;
	}

	EOS_Stats_IngestStatOptions Options = { };
	Options.ApiVersion = 3;
	UE_EOS_CHECK_API_MISMATCH(EOS_STATS_INGESTSTAT_API_LATEST, 3);
	Options.LocalUserId = LocalUserId;
	Options.TargetUserId = UserId;
	Options.Stats = EOSData.GetData();
	Options.StatsCount = EOSData.Num();

	FWriteStatsCallback* CallbackObj = new FWriteStatsCallback(FOnlineStatsEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this](const EOS_Stats_IngestStatCompleteCallbackInfo* Data)
	{
		bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
		if (!bWasSuccessful)
		{
			UE_LOG_ONLINE_STATS(Error, TEXT("EOS_Stats_IngestStat() failed with EOS result code (%s)"), *LexToString(Data->ResultCode));
		}
	};
	EOS_Stats_IngestStat(EOSSubsystem->StatsHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
}

void FOnlineStatsEOS::UpdateStats(const FUniqueNetIdRef LocalUserId, const TArray<FOnlineStatsUserUpdatedStats>& UpdatedUserStats, const FOnlineStatsUpdateStatsComplete& Delegate)
{
	EOS_ProductUserId UserId = nullptr;
	if (!IsRunningDedicatedServer()) // We'll keep the calling user as null for dedicated servers
	{
		const FUniqueNetIdEOS& EOSId = FUniqueNetIdEOS::Cast(*LocalUserId);
		UserId = EOSId.GetProductUserId();
		if (UserId == nullptr)
		{
			UE_LOG_ONLINE_STATS(Error, TEXT("UpdateStats() failed for unknown player (%s)"), *EOSId.ToDebugString());
			Delegate.ExecuteIfBound(FOnlineError(EOnlineErrorResult::InvalidCreds));
			return;
		}
	}

	if (UpdatedUserStats.Num() == 0 )
	{
		UE_LOG_ONLINE_STATS(Error, TEXT("UpdateStats() failed for player due to no user stats being specified"));
		Delegate.ExecuteIfBound(FOnlineError(EOnlineErrorResult::InvalidParams));
		return;
	}

	// Trigger one write for each user
	for (const FOnlineStatsUserUpdatedStats& StatsUpdate : UpdatedUserStats)
	{
		const FUniqueNetIdEOS& AccountEOSId = FUniqueNetIdEOS::Cast(*StatsUpdate.Account);
		const EOS_ProductUserId StatsUser = AccountEOSId.GetProductUserId();
		if (StatsUser != nullptr)
		{
			WriteStats(UserId, StatsUser, StatsUpdate);
		}
		else
		{
			UE_LOG_ONLINE_STATS(Error, TEXT("UpdateStats() failed for unknown player (%s)"), *StatsUpdate.Account->ToDebugString());
		}
	}
	// Mabye these wrote correctly...
	Delegate.ExecuteIfBound(FOnlineError(EOnlineErrorResult::Success));
}

#if !UE_BUILD_SHIPPING
void FOnlineStatsEOS::ResetStats(const FUniqueNetIdRef StatsUserId)
{
	UE_LOG_ONLINE_STATS(Warning, TEXT("ResetStats() is not supported"));
}
#endif

#endif
