// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineStatsInterface.h"
#include "OnlineSubsystemEOSTypes.h"

class FOnlineSubsystemEOS;

#if WITH_EOS_SDK
#include "eos_stats_types.h"

/**
 * Interface for interacting with EOS stats
 */
class FOnlineStatsEOS
	: public IOnlineStats
	, public TSharedFromThis<FOnlineStatsEOS, ESPMode::ThreadSafe>
{
public:
	FOnlineStatsEOS() = delete;
	virtual ~FOnlineStatsEOS() = default;

// IOnlineStats Interface
	virtual void QueryStats(const FUniqueNetIdRef LocalUserId, const FUniqueNetIdRef StatsUser, const FOnlineStatsQueryUserStatsComplete& Delegate) override;
	virtual void QueryStats(const FUniqueNetIdRef LocalUserId, const TArray<FUniqueNetIdRef>& StatUsers, const TArray<FString>& StatNames, const FOnlineStatsQueryUsersStatsComplete& Delegate) override;
	virtual TSharedPtr<const FOnlineStatsUserStats> GetStats(const FUniqueNetIdRef StatsUserId) const override;
	virtual void UpdateStats(const FUniqueNetIdRef LocalUserId, const TArray<FOnlineStatsUserUpdatedStats>& UpdatedUserStats, const FOnlineStatsUpdateStatsComplete& Delegate) override;
#if !UE_BUILD_SHIPPING
	virtual void ResetStats(const FUniqueNetIdRef StatsUserId) override;
#endif
// ~IOnlineStats Interface

	FOnlineStatsEOS(FOnlineSubsystemEOS* InSubsystem)
		: EOSSubsystem(InSubsystem)
	{
	}

private:
	void WriteStats(EOS_ProductUserId LocalUserId, EOS_ProductUserId UserId, const FOnlineStatsUserUpdatedStats& PlayerStats);

	/** Reference to the main EOS subsystem */
	FOnlineSubsystemEOS* EOSSubsystem;
	/** Cached list of stats for users as they arrive */
	TUniqueNetIdMap<TSharedRef<FOnlineStatsUserStats>> StatsCache;
};

typedef TSharedPtr<FOnlineStatsEOS, ESPMode::ThreadSafe> FOnlineStatsEOSPtr;
typedef TWeakPtr<FOnlineStatsEOS, ESPMode::ThreadSafe> FOnlineStatsEOSWeakPtr;

#endif
