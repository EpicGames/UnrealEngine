// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineStoreEOS.h"
#include "EOSSettings.h"
#include "IEOSSDKManager.h"

#if WITH_EOS_SDK

#include "IEOSSDKManager.h"
#include "OnlineSubsystemEOS.h"
#include "OnlineSubsystemEOSPrivate.h"
#include "UserManagerEOS.h"
#include "eos_ecom.h"

#define ONLINE_ERROR_NAMESPACE "com.epicgames.oss.eos.error"


FOnlineStoreEOS::FOnlineStoreEOS(FOnlineSubsystemEOS* InSubsystem)
	: EOSSubsystem(InSubsystem)
{
	check(EOSSubsystem != nullptr);
}

void FOnlineStoreEOS::QueryCategories(const FUniqueNetId& UserId, const FOnQueryOnlineStoreCategoriesComplete& Delegate)
{
	Delegate.ExecuteIfBound(false, TEXT("QueryCategories Not Implemented"));
}

void FOnlineStoreEOS::GetCategories(TArray<FOnlineStoreCategory>& OutCategories) const
{
	OutCategories.Reset();
}

void FOnlineStoreEOS::QueryOffersByFilter(const FUniqueNetId& UserId, const FOnlineStoreFilter& Filter, const FOnQueryOnlineStoreOffersComplete& Delegate)
{
	QueryOffers(UserId, Delegate);
}

void FOnlineStoreEOS::QueryOffersById(const FUniqueNetId& UserId, const TArray<FUniqueOfferId>& OfferIds, const FOnQueryOnlineStoreOffersComplete& Delegate)
{
	QueryOffers(UserId, Delegate);
}

typedef TEOSCallback<EOS_Ecom_OnQueryOffersCallback, EOS_Ecom_QueryOffersCallbackInfo, FOnlineStoreEOS> FQueryOffersCallback;

void FOnlineStoreEOS::QueryOffers(const FUniqueNetId& UserId, const FOnQueryOnlineStoreOffersComplete& Delegate)
{
	if (CachedOfferIds.Num() && CachedOffers.Num())
	{
		Delegate.ExecuteIfBound(true, CachedOfferIds, TEXT("Returning cached offers"));
		return;
	}
	const FUniqueNetIdEOS& UserEOSId = FUniqueNetIdEOS::Cast(UserId);
	const EOS_EpicAccountId AccountId = UserEOSId.GetEpicAccountId();
	if (AccountId == nullptr)
	{
		Delegate.ExecuteIfBound(false, TArray<FUniqueOfferId>(), TEXT("Can't query offers for a null user"));
		return;
	}

	CachedOfferIds.Reset();
	CachedOffers.Reset();

	EOS_Ecom_QueryOffersOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_QUERYOFFERS_API_LATEST, 1);
	Options.LocalUserId = AccountId;

	FQueryOffersCallback* CallbackObj = new FQueryOffersCallback(FOnlineStoreEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, OnComplete = FOnQueryOnlineStoreOffersComplete(Delegate)](const EOS_Ecom_QueryOffersCallbackInfo* Data)
	{
		EOS_EResult Result = Data->ResultCode;
		if (Result != EOS_EResult::EOS_Success)
		{
			OnComplete.ExecuteIfBound(false, CachedOfferIds, LexToString(Data->ResultCode));
			return;
		}

		EOS_Ecom_GetOfferCountOptions CountOptions = { };
		CountOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_GETOFFERCOUNT_API_LATEST, 1);
		CountOptions.LocalUserId = Data->LocalUserId;
		uint32 OfferCount = EOS_Ecom_GetOfferCount(EOSSubsystem->EcomHandle, &CountOptions);

		EOS_Ecom_CopyOfferByIndexOptions OfferOptions = { };
		OfferOptions.ApiVersion = 3;
		UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_COPYOFFERBYINDEX_API_LATEST, 3);
		OfferOptions.LocalUserId = Data->LocalUserId;
		// Iterate and parse the offer list
		for (uint32 OfferIndex = 0; OfferIndex < OfferCount; OfferIndex++)
		{
			EOS_Ecom_CatalogOffer* Offer = nullptr;
			OfferOptions.OfferIndex = OfferIndex;
			EOS_EResult OfferResult = EOS_Ecom_CopyOfferByIndex(EOSSubsystem->EcomHandle, &OfferOptions, &Offer);
			if (OfferResult != EOS_EResult::EOS_Success)
			{
				continue;
			}
			FOnlineStoreOfferRef OfferRef(new FOnlineStoreOffer());
			OfferRef->OfferId = Offer->Id;

			OfferRef->Title = FText::FromString(Offer->TitleText);
			OfferRef->Description = FText::FromString(Offer->DescriptionText);
			OfferRef->LongDescription = FText::FromString(Offer->LongDescriptionText);

			OfferRef->ExpirationDate = FDateTime(Offer->ExpirationTimestamp);

			OfferRef->CurrencyCode = Offer->CurrencyCode;
			if (Offer->PriceResult == EOS_EResult::EOS_Success)
			{
#if EOS_ECOM_CATALOGOFFER_API_LATEST >= 3
				OfferRef->RegularPrice = Offer->OriginalPrice64;
				OfferRef->NumericPrice = Offer->CurrentPrice64;
#else
				OfferRef->RegularPrice = Offer->OriginalPrice;
				OfferRef->NumericPrice = Offer->CurrentPrice;
#endif
				OfferRef->DiscountType = Offer->DiscountPercentage == 100 ? EOnlineStoreOfferDiscountType::NotOnSale : EOnlineStoreOfferDiscountType::DiscountAmount;
			}

			OfferRef->RegularPriceText = FText::AsCurrencyBase(OfferRef->RegularPrice, OfferRef->CurrencyCode, NULL, Offer->DecimalPoint);
			OfferRef->PriceText = FText::AsCurrencyBase(OfferRef->NumericPrice, OfferRef->CurrencyCode, NULL, Offer->DecimalPoint);

			CachedOffers.Add(OfferRef);
			CachedOfferIds.Add(OfferRef->OfferId);

			EOS_Ecom_CatalogOffer_Release(Offer);
		}

		OnComplete.ExecuteIfBound(true, CachedOfferIds, TEXT(""));
	};
	EOS_Ecom_QueryOffers(EOSSubsystem->EcomHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
}

void FOnlineStoreEOS::GetOffers(TArray<FOnlineStoreOfferRef>& OutOffers) const
{
	OutOffers = CachedOffers;
}

TSharedPtr<FOnlineStoreOffer> FOnlineStoreEOS::GetOffer(const FUniqueOfferId& OfferId) const
{
	for (FOnlineStoreOfferRef Offer : CachedOffers)
	{
		if (Offer->OfferId == OfferId)
		{
			return Offer;
		}
	}
	return nullptr;
}

typedef TEOSCallback<EOS_Ecom_OnCheckoutCallback, EOS_Ecom_CheckoutCallbackInfo, FOnlineStoreEOS> FCheckoutCallback;

void FOnlineStoreEOS::Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseCheckoutComplete& Delegate)
{
	const FUniqueNetIdEOS& UserEOSId = FUniqueNetIdEOS::Cast(UserId);
	const EOS_EpicAccountId AccountId = UserEOSId.GetEpicAccountId();
	if (AccountId == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("Checkout: failed due to invalid user"));
		Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::InvalidUser), MakeShared<FPurchaseReceipt>());
		return;
	}
	if (CheckoutRequest.PurchaseOffers.Num() == 0)
	{
		UE_LOG_ONLINE(Error, TEXT("Checkout: failed due to no items to buy"));
		Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::InvalidParams), MakeShared<FPurchaseReceipt>());
		return;
	}
	if (CheckoutRequest.PurchaseOffers.Num() > EOS_ECOM_CHECKOUT_MAX_ENTRIES)
	{
		UE_LOG_ONLINE(Error, TEXT("Checkout: can only buy %d items at a time"), EOS_ECOM_CHECKOUT_MAX_ENTRIES);
		Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::InvalidParams), MakeShared<FPurchaseReceipt>());
		return;
	}

	const int32 NumItems = CheckoutRequest.PurchaseOffers.Num();
	char Items[EOS_ECOM_CHECKOUT_MAX_ENTRIES][EOS_ECOM_TRANSACTIONID_MAXIMUM_LENGTH];
	TArray<EOS_Ecom_CheckoutEntry> Entries;
	Entries.AddZeroed(NumItems);

	// Convert the items
	for (int32 Index = 0; Index < NumItems; Index++)
	{
		Entries[Index].ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_CHECKOUTENTRY_API_LATEST, 1);
		Entries[Index].OfferId = Items[Index];
		FCStringAnsi::Strncpy(Items[Index], TCHAR_TO_UTF8(*CheckoutRequest.PurchaseOffers[Index].OfferId), EOS_ECOM_TRANSACTIONID_MAXIMUM_LENGTH);
	}

	EOS_Ecom_CheckoutOptions Options = { };
	Options.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_CHECKOUT_API_LATEST, 2);
	Options.LocalUserId = AccountId;
	Options.EntryCount = NumItems;
	Options.Entries = (const EOS_Ecom_CheckoutEntry*)Entries.GetData();
	Options.OverrideCatalogNamespace = nullptr;
	Options.PreferredOrientation = EOS_ECheckoutOrientation::EOS_ECO_Default;

	FCheckoutCallback* CallbackObj = new FCheckoutCallback(FOnlineStoreEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, OnComplete = FOnPurchaseCheckoutComplete(Delegate)](const EOS_Ecom_CheckoutCallbackInfo* Data)
	{
		const EOS_EResult CheckoutResult = Data->ResultCode;
		if (CheckoutResult != EOS_EResult::EOS_Success)
		{
			if (CheckoutResult == EOS_EResult::EOS_Canceled)
			{
				UE_LOG_ONLINE(Log, TEXT("EOS_Ecom_Checkout was canceled by user."));
				OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Canceled), MakeShared<FPurchaseReceipt>());
			}
			else if (CheckoutResult == EOS_EResult::EOS_Ecom_PurchaseProcessing)
			{
				// See header documentation for EOS_Ecom_PurchaseProcessing and online EOS documentation for more information
				UE_LOG_ONLINE(Log, TEXT("QueryReceipts will need to be called to confirm the purchase. This can be done via a UI button."));
				OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::FailExtended, LexToString(CheckoutResult)), MakeShared<FPurchaseReceipt>());
			}
			else
			{
				UE_LOG_ONLINE(Error, TEXT("EOS_Ecom_Checkout: failed with error (%s)"), *LexToString(CheckoutResult));
				OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Unknown), MakeShared<FPurchaseReceipt>());
			}
			return;
		}

		EOS_Ecom_CopyTransactionByIdOptions CopyTransactionOptions = { };
		CopyTransactionOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_COPYTRANSACTIONBYID_API_LATEST, 1);
		CopyTransactionOptions.LocalUserId = Data->LocalUserId;
		CopyTransactionOptions.TransactionId = Data->TransactionId;
		EOS_Ecom_HTransaction TransactionHandle;
		const EOS_EResult CopyTransactionResult = EOS_Ecom_CopyTransactionById(EOSSubsystem->EcomHandle, &CopyTransactionOptions, &TransactionHandle);
		if (CopyTransactionResult != EOS_EResult::EOS_Success)
		{
			OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Unknown), MakeShared<FPurchaseReceipt>());
			return;
		}

		TSet<FString> EntitlementIds = { };
		EOS_Ecom_Transaction_GetEntitlementsCountOptions EntitlementCountOptions = { };
		EntitlementCountOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_TRANSACTION_GETENTITLEMENTSCOUNT_API_LATEST, 1);
		const uint32_t EntitlementsCount = EOS_Ecom_Transaction_GetEntitlementsCount(TransactionHandle, &EntitlementCountOptions);
		for (uint32_t i = 0; i < EntitlementsCount; i++)
		{
			EOS_Ecom_Transaction_CopyEntitlementByIndexOptions Options = { };
			Options.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_TRANSACTION_COPYENTITLEMENTBYINDEX_API_LATEST, 1);
			EOS_Ecom_Entitlement* Entitlement;
			const EOS_EResult CopyEntitlementResult = EOS_Ecom_Transaction_CopyEntitlementByIndex(TransactionHandle, &Options, &Entitlement);
			if (CopyEntitlementResult == EOS_EResult::EOS_Success)
			{
				EntitlementIds.Add(FString(UTF8_TO_TCHAR(Entitlement->EntitlementId)));
				EOS_Ecom_Entitlement_Release(Entitlement);
			}
		}
		EOS_Ecom_Transaction_Release(TransactionHandle);

		// Update the cached receipts
		QueryReceipts(*EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(Data->LocalUserId), true,
			FOnQueryReceiptsComplete::CreateLambda([this, PurchaseComplete = FOnPurchaseCheckoutComplete(OnComplete), EntitlementIds = MoveTemp(EntitlementIds)](const FOnlineError& Result)
		{
			if (!Result.WasSuccessful())
			{
				UE_LOG_ONLINE(Error, TEXT("EOS_Ecom_Checkout: failed to query receipts after purchase"));
				PurchaseComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Unknown), MakeShared<FPurchaseReceipt>());
				return;
			}

			TSharedRef<FPurchaseReceipt> Receipt = MakeShared<FPurchaseReceipt>();
			// Find the transaction in our receipts
			for (const FPurchaseReceipt& SearchReceipt : CachedReceipts)
			{
				if (EntitlementIds.Contains(SearchReceipt.TransactionId))
				{
					Receipt = MakeShared<FPurchaseReceipt>(SearchReceipt);
					break;
				}
			}
			PurchaseComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Success), Receipt);
		}));

	};
	EOS_Ecom_Checkout(EOSSubsystem->EcomHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
}

void FOnlineStoreEOS::Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseReceiptlessCheckoutComplete& Delegate)
{
	// Checkout with no receipt query Delegate is not implemented, please use the other Checkout method
	Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::NotImplemented));
}

void FOnlineStoreEOS::FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId)
{
	UE_LOG_ONLINE(Verbose, TEXT("FinalizePurchase: Not supported. Did you mean FinalizeReceiptValidationInfo?"));
}

void FOnlineStoreEOS::RedeemCode(const FUniqueNetId& UserId, const FRedeemCodeRequest& RedeemCodeRequest, const FOnPurchaseRedeemCodeComplete& Delegate)
{
	static const TSharedRef<FPurchaseReceipt> BlankReceipt(MakeShared<FPurchaseReceipt>());
	Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::NotImplemented), BlankReceipt);
}

typedef TEOSCallback<EOS_Ecom_OnQueryEntitlementsCallback, EOS_Ecom_QueryEntitlementsCallbackInfo, FOnlineStoreEOS> FQueryReceiptsCallback;
typedef TEOSCallback<EOS_Ecom_OnQueryOwnershipBySandboxIdsCallback, EOS_Ecom_QueryOwnershipBySandboxIdsCallbackInfo, FOnlineStoreEOS> FQueryOwnershipBySandboxIdsCallback;

void FOnlineStoreEOS::QueryReceipts(const FUniqueNetId& UserId, bool bRestoreReceipts, const FOnQueryReceiptsComplete& Delegate)
{
	const FUniqueNetIdEOS& UserEOSId = FUniqueNetIdEOS::Cast(UserId);
	const EOS_EpicAccountId AccountId = UserEOSId.GetEpicAccountId();
	if (AccountId == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("QueryReceipts: failed due to invalid user"));
		Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::InvalidUser));
		return;
	}

	CachedReceipts.Reset();

	// First we'll call QueryEntitlements to query ownership for individual products

	EOS_Ecom_QueryEntitlementsOptions Options = { };
	Options.ApiVersion = 3;
	UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_QUERYENTITLEMENTS_API_LATEST, 3);
	Options.LocalUserId = AccountId;
	Options.bIncludeRedeemed = bRestoreReceipts ? EOS_TRUE : EOS_FALSE;
	Options.OverrideCatalogNamespace = nullptr;

	FQueryReceiptsCallback* CallbackObj = new FQueryReceiptsCallback(FOnlineStoreEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, OnComplete = FOnQueryReceiptsComplete(Delegate)](const EOS_Ecom_QueryEntitlementsCallbackInfo* Data)
	{
		EOS_EResult Result = Data->ResultCode;
		if (Result != EOS_EResult::EOS_Success)
		{
			UE_LOG_ONLINE(Error, TEXT("EOS_Ecom_QueryEntitlements: failed with error (%s)"), *LexToString(Data->ResultCode));
			OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Unknown));
			return;
		}

		EOS_Ecom_GetEntitlementsCountOptions CountOptions = { };
		CountOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_GETENTITLEMENTSCOUNT_API_LATEST, 1);
		CountOptions.LocalUserId = Data->LocalUserId;
		uint32 Count = EOS_Ecom_GetEntitlementsCount(EOSSubsystem->EcomHandle, &CountOptions);
		CachedReceipts.AddZeroed(Count);

		EOS_Ecom_CopyEntitlementByIndexOptions CopyOptions = { };
		CopyOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_COPYENTITLEMENTBYINDEX_API_LATEST, 1);
		CopyOptions.LocalUserId = Data->LocalUserId;

		for (uint32 Index = 0; Index < Count; Index++)
		{
			CopyOptions.EntitlementIndex = Index;

			EOS_Ecom_Entitlement* Receipt = nullptr;
			EOS_EResult CopyResult = EOS_Ecom_CopyEntitlementByIndex(EOSSubsystem->EcomHandle, &CopyOptions, &Receipt);
			if (CopyResult != EOS_EResult::EOS_Success && CopyResult != EOS_EResult::EOS_Ecom_EntitlementStale)
			{
				UE_LOG_ONLINE(Error, TEXT("EOS_Ecom_CopyEntitlementByIndex: failed with error (%s)"), *LexToString(CopyResult));
				continue;
			}

			// Parse the entitlement into the receipt format
			FPurchaseReceipt& PurchaseReceipt = CachedReceipts[Index];
			PurchaseReceipt.TransactionId = Receipt->EntitlementId;
			PurchaseReceipt.TransactionState = EPurchaseTransactionState::Purchased;
			PurchaseReceipt.AddReceiptOffer(FOfferNamespace(), Receipt->CatalogItemId, 1);
			FPurchaseReceipt::FLineItemInfo& LineItem = PurchaseReceipt.ReceiptOffers[0].LineItems.Emplace_GetRef();
			LineItem.ItemName = Receipt->EntitlementName;
			LineItem.UniqueId = Receipt->EntitlementId;
			LineItem.ValidationInfo = Receipt->bRedeemed == EOS_TRUE ? "" : Receipt->EntitlementId;

			EOS_Ecom_Entitlement_Release(Receipt);
		}

		// After QueryEntitlements is done, we call QueryOwnership to query ownership granted by other services like season passes, any benefits of which would not have been returned by QueryEntitlements

		// Retrieve the SandboxId from config
		FString SandboxIdString;
		const FEOSSettings& EOSSettings = UEOSSettings::GetSettings();
		if (EOSSettings.bUseNamedPlatformConfig)
		{
			const FEOSSDKPlatformConfig* const PlatformConfig = EOSSubsystem->EOSSDKManager->GetPlatformConfig(EOSSettings.PlatformConfigName);
			if (!PlatformConfig)
			{
				UE_LOG_ONLINE(Warning, TEXT("QueryReceipts - GetPlatformConfig failed - EOS_Ecom_QueryOwnershipBySandboxIds not called"));
				OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Unknown));
				return;
			}
			SandboxIdString = PlatformConfig->SandboxId; 
		}
		else
		{
			FEOSArtifactSettings ArtifactSettings;
			if (!UEOSSettings::GetSelectedArtifactSettings(ArtifactSettings))
			{
				UE_LOG_ONLINE(Warning, TEXT("QueryReceipts - GetSelectedArtifactSettings failed - EOS_Ecom_QueryOwnershipBySandboxIds not called"));
				OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Unknown));
				return;
			}
			SandboxIdString = ArtifactSettings.SandboxId;
		}

		if (SandboxIdString.IsEmpty())
		{
			UE_LOG_ONLINE(Warning, TEXT("QueryReceipts - Empty SandboxId - EOS_Ecom_QueryOwnershipBySandboxIds not called"));
			OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Unknown));
			return;
		}

		EOS_Ecom_QueryOwnershipBySandboxIdsOptions Options = {};
		Options.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_QUERYOWNERSHIPBYSANDBOXIDSOPTIONS_API_LATEST, 1);
		Options.LocalUserId = Data->LocalUserId;
		Options.SandboxIdsCount = 1;
		const auto SandboxIdUtf8 = StringCast<UTF8CHAR>(*SandboxIdString);
		const char* SandboxId = (const char*)SandboxIdUtf8.Get();
		Options.SandboxIds = &SandboxId;

		FQueryOwnershipBySandboxIdsCallback* CallbackObj = new FQueryOwnershipBySandboxIdsCallback(FOnlineStoreEOSWeakPtr(AsShared()));
		CallbackObj->CallbackLambda = [this, OnComplete = FOnQueryReceiptsComplete(OnComplete)](const EOS_Ecom_QueryOwnershipBySandboxIdsCallbackInfo* Data)
		{
			const EOS_EResult Result = Data->ResultCode;
			if (Result != EOS_EResult::EOS_Success)
			{
				UE_LOG_ONLINE(Error, TEXT("EOS_Ecom_QueryOwnership: failed with error (%s)"), *LexToString(Data->ResultCode));
				OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Unknown));
				return;
			}
			
			UE_LOG_ONLINE(VeryVerbose, TEXT("Sandbox Id Ownership Count: %d"), Data->SandboxIdItemOwnershipsCount);

			for (const EOS_Ecom_SandboxIdItemOwnership& SandboxOwnershipEntry : TArrayView<const EOS_Ecom_SandboxIdItemOwnership>(Data->SandboxIdItemOwnerships, Data->SandboxIdItemOwnershipsCount))
			{
				UE_LOG_ONLINE(VeryVerbose, TEXT("	Sandbox Id: %hs"), SandboxOwnershipEntry.SandboxId);
				UE_LOG_ONLINE(VeryVerbose, TEXT("	Ownership Count: %d"), SandboxOwnershipEntry.OwnedCatalogItemIdsCount);

				for (uint32_t CatalogItemIndex = 0; CatalogItemIndex < SandboxOwnershipEntry.OwnedCatalogItemIdsCount; CatalogItemIndex++)
				{
					const EOS_Ecom_CatalogItemId& CatalogItemId = SandboxOwnershipEntry.OwnedCatalogItemIds[CatalogItemIndex];

					FPurchaseReceipt& PurchaseReceipt = CachedReceipts.Emplace_GetRef();
					PurchaseReceipt.TransactionState = EPurchaseTransactionState::Purchased;
					PurchaseReceipt.TransactionId = FString(CatalogItemId);

					UE_LOG_ONLINE(VeryVerbose, TEXT("		Catalog Item %d"), CatalogItemIndex);
					UE_LOG_ONLINE(VeryVerbose, TEXT("			Catalog Item Id: %hs"), CatalogItemId);
				}
			}
			
			OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Success));
		};

		EOS_Ecom_QueryOwnershipBySandboxIds(EOSSubsystem->EcomHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
	};
	EOS_Ecom_QueryEntitlements(EOSSubsystem->EcomHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
}

void FOnlineStoreEOS::GetReceipts(const FUniqueNetId& UserId, TArray<FPurchaseReceipt>& OutReceipts) const
{
	OutReceipts = CachedReceipts;
}

typedef TEOSCallback<EOS_Ecom_OnRedeemEntitlementsCallback, EOS_Ecom_RedeemEntitlementsCallbackInfo, FOnlineStoreEOS> FRedeemReceiptCallback;

void FOnlineStoreEOS::FinalizeReceiptValidationInfo(const FUniqueNetId& UserId, FString& InReceiptValidationInfo, const FOnFinalizeReceiptValidationInfoComplete& Delegate)
{
	const FUniqueNetIdEOS& UserEOSId = FUniqueNetIdEOS::Cast(UserId);
	const EOS_EpicAccountId AccountId = UserEOSId.GetEpicAccountId();
	if (AccountId == nullptr)
	{
		Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::InvalidUser), InReceiptValidationInfo);
		return;
	}
	if (InReceiptValidationInfo.IsEmpty())
	{
		Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::InvalidParams), InReceiptValidationInfo);
		return;
	}

	char const* Ids[1];
	FTCHARToUTF8 EntitlementId(*InReceiptValidationInfo);
	Ids[0] = EntitlementId.Get();

	EOS_Ecom_RedeemEntitlementsOptions Options = { };
	Options.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_REDEEMENTITLEMENTS_API_LATEST, 2);
	Options.LocalUserId = AccountId;
	Options.EntitlementIdCount = 1;
	Options.EntitlementIds = Ids;

	FRedeemReceiptCallback* CallbackObj = new FRedeemReceiptCallback(FOnlineStoreEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, Info = FString(InReceiptValidationInfo), OnComplete = FOnFinalizeReceiptValidationInfoComplete(Delegate)](const EOS_Ecom_RedeemEntitlementsCallbackInfo* Data)
	{
		EOS_EResult Result = Data->ResultCode;
		if (Result != EOS_EResult::EOS_Success)
		{
			UE_LOG_ONLINE(Error, TEXT("EOS_Ecom_RedeemEntitlements: failed with error (%s)"), *LexToString(Data->ResultCode));
			OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Unknown), Info);
			return;
		}

		// Find the receipt in our list and mark as redeemed (clear the validation info)
		for (FPurchaseReceipt& SearchReceipt : CachedReceipts)
		{
			if (SearchReceipt.TransactionId == Info)
			{
				// Clearing this field tells the game it can't be redeemed
				SearchReceipt.ReceiptOffers[0].LineItems[0].ValidationInfo.Empty();
				break;
			}
		}

		OnComplete.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Success), Info);
	};
	EOS_Ecom_RedeemEntitlements(EOSSubsystem->EcomHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
}

bool FOnlineStoreEOS::HandleEcomExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("OFFERS")))
	{
		QueryOffers(*EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(EOSSubsystem->UserManager->GetDefaultLocalUser()),
			FOnQueryOnlineStoreOffersComplete::CreateLambda([this](bool bWasSuccessful, const TArray<FUniqueOfferId>& OfferIds, const FString& ErrorStr)
		{
			UE_LOG_ONLINE(Error, TEXT("QueryOffers: %s with error (%s)"), bWasSuccessful ? TEXT("succeeded") : TEXT("failed"), *ErrorStr);

			for (const FUniqueOfferId& OfferId : OfferIds)
			{
				UE_LOG_ONLINE(Log, TEXT("OfferId: %s"), *OfferId);
			}
		}));
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("RECEIPTS")))
	{
		QueryReceipts(*EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(EOSSubsystem->UserManager->GetDefaultLocalUser()), false,
			FOnQueryReceiptsComplete::CreateLambda([this](const FOnlineError& Result)
		{
			UE_LOG_ONLINE(Log, TEXT("QueryReceipts: %s with error (%s)"), Result.WasSuccessful() ? TEXT("succeeded") : TEXT("failed"), *Result.GetErrorRaw());
			for (const FPurchaseReceipt& Receipt : CachedReceipts)
			{
				UE_LOG_ONLINE(Log, TEXT("Receipt: %s"), *Receipt.TransactionId);
				UE_LOG_ONLINE(Log, TEXT("\tOffer Id (%s), Quantity (%d)"), *Receipt.ReceiptOffers[0].OfferId, Receipt.ReceiptOffers[0].Quantity);
				UE_LOG_ONLINE(Log, TEXT("\tLine item (%s) is %s"), *Receipt.ReceiptOffers[0].LineItems[0].ItemName, Receipt.ReceiptOffers[0].LineItems[0].IsRedeemable() ? TEXT("redeemable") : TEXT("not redeemable"));
			}
		}));
		return true;
	}
	return false;
}

#endif