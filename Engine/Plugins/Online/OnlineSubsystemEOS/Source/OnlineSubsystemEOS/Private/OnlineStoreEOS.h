// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Online/CoreOnline.h"
#include "Interfaces/OnlineStoreInterfaceV2.h"
#include "Interfaces/OnlinePurchaseInterface.h"
#include "OnlineSubsystemEOSTypes.h"

#if WITH_EOS_SDK
	#include "eos_ecom_types.h"

class UWorld;

/**
 * Implementation for online store via EGS
 */
class FOnlineStoreEOS :
	public IOnlineStoreV2,
	public IOnlinePurchase,
	public TSharedFromThis<FOnlineStoreEOS, ESPMode::ThreadSafe>
{
public:
	virtual ~FOnlineStoreEOS() = default;

// Begin IOnlineStoreV2
	virtual void QueryCategories(const FUniqueNetId& UserId, const FOnQueryOnlineStoreCategoriesComplete& Delegate) override;
	virtual void GetCategories(TArray<FOnlineStoreCategory>& OutCategories) const override;
	virtual void QueryOffersByFilter(const FUniqueNetId& UserId, const FOnlineStoreFilter& Filter, const FOnQueryOnlineStoreOffersComplete& Delegate) override;
	virtual void QueryOffersById(const FUniqueNetId& UserId, const TArray<FUniqueOfferId>& OfferIds, const FOnQueryOnlineStoreOffersComplete& Delegate) override;
	virtual void GetOffers(TArray<FOnlineStoreOfferRef>& OutOffers) const override;
	virtual TSharedPtr<FOnlineStoreOffer> GetOffer(const FUniqueOfferId& OfferId) const override;
// End IOnlineStoreV2

// Begin IOnlinePurchase
	virtual bool IsAllowedToPurchase(const FUniqueNetId& UserId) override { return true; }
	virtual bool IsAllowedToPurchase(const FUniqueNetId& UserId, FOnlineError& Error) override 
	{ 
		Error = FOnlineError(EOnlineErrorResult::Success);
		return true; 
	}

	virtual void Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseCheckoutComplete& Delegate) override;
	virtual void Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseReceiptlessCheckoutComplete& Delegate) override;
	virtual void FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId) override;
	virtual void RedeemCode(const FUniqueNetId& UserId, const FRedeemCodeRequest& RedeemCodeRequest, const FOnPurchaseRedeemCodeComplete& Delegate) override;
	virtual void QueryReceipts(const FUniqueNetId& UserId, bool bRestoreReceipts, const FOnQueryReceiptsComplete& Delegate) override;
	virtual void GetReceipts(const FUniqueNetId& UserId, TArray<FPurchaseReceipt>& OutReceipts) const override;
	virtual void FinalizeReceiptValidationInfo(const FUniqueNetId& UserId, FString& InReceiptValidationInfo, const FOnFinalizeReceiptValidationInfoComplete& Delegate) override;
// End IOnlinePurchase

	FOnlineStoreEOS(FOnlineSubsystemEOS* InSubsystem);

	bool HandleEcomExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);

private:
	/** Default constructor disabled */
	FOnlineStoreEOS() = delete;

	void QueryOffers(const FUniqueNetId& UserId, const FOnQueryOnlineStoreOffersComplete& Delegate);

	/** Reference to the main EOS subsystem */
	FOnlineSubsystemEOS* EOSSubsystem;

	/** The set of offers for this title */
	TArray<FOnlineStoreOfferRef> CachedOffers;
	/** List of offer ids for this title */
	TArray<FUniqueOfferId> CachedOfferIds;

	/** List of receipts for the user */
//@todo joeg - make this support multiple users
	TArray<FPurchaseReceipt> CachedReceipts;
};

typedef TSharedPtr<FOnlineStoreEOS, ESPMode::ThreadSafe> FOnlineStoreEOSPtr;
typedef TWeakPtr<FOnlineStoreEOS, ESPMode::ThreadSafe> FOnlineStoreEOSWeakPtr;

#endif
