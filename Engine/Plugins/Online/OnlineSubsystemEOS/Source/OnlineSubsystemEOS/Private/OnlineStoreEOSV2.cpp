// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineStoreEOSV2.h"
#include "EOSSettings.h"
#include "IEOSSDKManager.h"

#if WITH_EOS_SDK

#include "OnlineSubsystemEOS.h"
#include "OnlineSubsystemEOSPrivate.h"
#include "UserManagerEOS.h"
#include "eos_ecom.h"

#define ONLINE_ERROR_NAMESPACE "com.epicgames.oss.eos.error"


FOnlineStoreEOSV2::FOnlineStoreEOSV2(FOnlineSubsystemEOS* InSubsystem)
	: EOSSubsystem(InSubsystem)
{
	check(EOSSubsystem != nullptr);
}

void FOnlineStoreEOSV2::QueryCategories(const FUniqueNetId& UserId, const FOnQueryOnlineStoreCategoriesComplete& Delegate)
{
	UE_LOG_ONLINE(Verbose, TEXT("[%hs] Not implemented"), __FUNCTION__);
	Delegate.ExecuteIfBound(false, "Not implemented");
}

void FOnlineStoreEOSV2::GetCategories(TArray<FOnlineStoreCategory>& OutCategories) const
{
	UE_LOG_ONLINE(Verbose, TEXT("[%hs] Not implemented"), __FUNCTION__);
	OutCategories.Reset();
}

void FOnlineStoreEOSV2::QueryOffersByFilter(const FUniqueNetId& UserId, const FOnlineStoreFilter& Filter, const FOnQueryOnlineStoreOffersComplete& Delegate)
{
	UE_LOG_ONLINE(Verbose, TEXT("[%hs] Not implemented - No EOS SDK API to query offers by filter - Querying all offers"), __FUNCTION__);
	QueryOffers(UserId, Delegate);
}

void FOnlineStoreEOSV2::QueryOffersById(const FUniqueNetId& UserId, const TArray<FUniqueOfferId>& OfferIds, const FOnQueryOnlineStoreOffersComplete& Delegate)
{
	UE_LOG_ONLINE(Verbose, TEXT("[%hs] Not implemented - No EOS SDK API to query offers by Id - Querying all offers"), __FUNCTION__);
	QueryOffers(UserId, Delegate);
}

typedef TEOSCallback<EOS_Ecom_OnQueryOffersCallback, EOS_Ecom_QueryOffersCallbackInfo, FOnlineStoreEOSV2> FOnlineStoreV2QueryOffersCallback;

void FOnlineStoreEOSV2::QueryOffers(const FUniqueNetId& UserId, const FOnQueryOnlineStoreOffersComplete& Delegate)
{
	const FUniqueNetIdEOS& UserEOSId = FUniqueNetIdEOS::Cast(UserId);
	const EOS_EpicAccountId AccountId = UserEOSId.GetEpicAccountId();
	if (AccountId == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("[%hs] EpicAccountId is null"), __FUNCTION__);
		Delegate.ExecuteIfBound(false, TArray<FUniqueOfferId>(), "EpicAccountId is null");
		return;
	}

	EOS_Ecom_QueryOffersOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_QUERYOFFERS_API_LATEST, 1);
	Options.LocalUserId = AccountId;

	FOnlineStoreV2QueryOffersCallback* CallbackObj = new FOnlineStoreV2QueryOffersCallback(FOnlineStoreEOSV2WeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this, OnComplete = FOnQueryOnlineStoreOffersComplete(Delegate)](const EOS_Ecom_QueryOffersCallbackInfo* Data)
		{
			CachedOfferIds.Reset();
			CachedOffers.Reset();

			if (Data->ResultCode == EOS_EResult::EOS_Success)
			{
				EOS_Ecom_GetOfferCountOptions CountOptions = { };
				CountOptions.ApiVersion = 1;
				UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_GETOFFERCOUNT_API_LATEST, 1);
				CountOptions.LocalUserId = Data->LocalUserId;
				uint32 OfferCount = EOS_Ecom_GetOfferCount(EOSSubsystem->EcomHandle, &CountOptions);

				EOS_Ecom_CopyOfferByIndexOptions OfferOptions = { };
				OfferOptions.ApiVersion = 3;
				UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_COPYOFFERBYINDEX_API_LATEST, 3);
				OfferOptions.LocalUserId = Data->LocalUserId;

				for (uint32 OfferIndex = 0; OfferIndex < OfferCount; OfferIndex++)
				{
					EOS_Ecom_CatalogOffer* Offer = nullptr;
					OfferOptions.OfferIndex = OfferIndex;
					EOS_EResult OfferResult = EOS_Ecom_CopyOfferByIndex(EOSSubsystem->EcomHandle, &OfferOptions, &Offer);
					if (OfferResult == EOS_EResult::EOS_Success)
					{
						FOnlineStoreOfferRef OfferRef = MakeShared<FOnlineStoreOffer>();
						//OfferId represents an EOS_Ecom_CatalogOfferId
						OfferRef->OfferId = Offer->Id ? FUniqueOfferId(StringCast<TCHAR>(Offer->Id).Get()) : FString();
						OfferRef->Title = FText::FromString(Offer->TitleText ? StringCast<TCHAR>(Offer->TitleText).Get() : FString());
						OfferRef->Description = FText::FromString(Offer->DescriptionText ? StringCast<TCHAR>(Offer->DescriptionText).Get() : FString());
						OfferRef->LongDescription = FText::FromString(Offer->LongDescriptionText ? StringCast<TCHAR>(Offer->LongDescriptionText).Get() : FString());
						OfferRef->ExpirationDate = FDateTime::FromUnixTimestamp(Offer->ExpirationTimestamp);
						OfferRef->CurrencyCode = Offer->CurrencyCode ? StringCast<TCHAR>(Offer->CurrencyCode).Get() : FString();
						OfferRef->DynamicFields.Add(TEXT("bAvailableForPurchase"), LexToString(Offer->bAvailableForPurchase));
						OfferRef->DynamicFields.Add(TEXT("PurchaseLimit"), LexToString(Offer->PurchaseLimit));

						if (Offer->PriceResult == EOS_EResult::EOS_Success)
						{
#if EOS_ECOM_CATALOGOFFER_API_LATEST >= 3
							OfferRef->RegularPrice = Offer->OriginalPrice64;
							OfferRef->NumericPrice = Offer->CurrentPrice64;
#else
							OfferRef->RegularPrice = Offer->OriginalPrice;
							OfferRef->NumericPrice = Offer->CurrentPrice;
#endif
							OfferRef->DiscountType = Offer->DiscountPercentage == 0 ? EOnlineStoreOfferDiscountType::NotOnSale : EOnlineStoreOfferDiscountType::DiscountAmount;
							OfferRef->RegularPriceText = FText::AsCurrencyBase(OfferRef->RegularPrice, OfferRef->CurrencyCode, NULL, Offer->DecimalPoint);
							OfferRef->PriceText = FText::AsCurrencyBase(OfferRef->NumericPrice, OfferRef->CurrencyCode, NULL, Offer->DecimalPoint);
						}
						else
						{
							UE_LOG_ONLINE(Verbose, TEXT("[FOnlineStoreEOSV2::QueryOffers] Price Result %s Offer %s"), *LexToString(Offer->PriceResult), *OfferRef->OfferId);
						}

						CachedOffers.Add(OfferRef);
						CachedOfferIds.Add(OfferRef->OfferId);

						EOS_Ecom_CatalogOffer_Release(Offer);
					}
					else
					{
						UE_LOG_ONLINE(Warning, TEXT("[FOnlineStoreEOSV2::QueryOffers] EOS_Ecom_CopyOfferByIndex result code (%s)"), *LexToString(OfferResult));
						continue;
					}
				}
				OnComplete.ExecuteIfBound(true, CachedOfferIds, TEXT(""));
			}
			else
			{
				FString ErrorString = FString::Printf(TEXT("result code (%s)"), *LexToString(Data->ResultCode));
				UE_LOG_ONLINE(Error, TEXT("[FOnlineStoreEOSV2::QueryOffers] %s"), *ErrorString);
				OnComplete.ExecuteIfBound(false, CachedOfferIds, ErrorString);
			}
		};
	EOS_Ecom_QueryOffers(EOSSubsystem->EcomHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
}

void FOnlineStoreEOSV2::GetOffers(TArray<FOnlineStoreOfferRef>& OutOffers) const
{
	OutOffers.Empty();

	if (CachedOffers.IsEmpty())
	{
		UE_LOG_ONLINE(Verbose, TEXT("[%hs] CachedOffers empty, call QueryOffers"), __FUNCTION__);
		return;
	}

	OutOffers = CachedOffers;
}

TSharedPtr<FOnlineStoreOffer> FOnlineStoreEOSV2::GetOffer(const FUniqueOfferId& OfferId) const
{
	for (FOnlineStoreOfferRef Offer : CachedOffers)
	{
		if (Offer->OfferId == OfferId)
		{
			return Offer;
		}
	}
	
	UE_LOG_ONLINE(Verbose, TEXT("[%hs] CachedOffers does not contain an offer with OfferId (%s)"), __FUNCTION__,*OfferId);
	return nullptr;
}

bool FOnlineStoreEOSV2::HandleEcomExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (!EOSSubsystem || !EOSSubsystem->UserManager.IsValid() || !EOSSubsystem->StoreInterfaceV2Ptr.IsValid())
	{
		UE_LOG_ONLINE(Error, TEXT("[%hs] EOSSubsystem or UserManager or StoreInterfaceV2 invalid"), __FUNCTION__);
		return false;
	}

	if (FParse::Command(&Cmd, TEXT("OFFERS")))
	{
		QueryOffers(*EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(EOSSubsystem->UserManager->GetDefaultLocalUser()),
			FOnQueryOnlineStoreOffersComplete::CreateLambda([this](bool bWasSuccessful, const TArray<FUniqueOfferId>& OfferIds, const FString& ErrorStr)
				{
					if (bWasSuccessful)
					{
						UE_LOG_ONLINE(Log, TEXT("[FOnlineStoreEOSV2::HandleEcomExec] QueryOffers success"));
						for (const FUniqueOfferId& OfferId : OfferIds)
						{
							UE_LOG_ONLINE(Log, TEXT("[FOnlineStoreEOSV2::HandleEcomExec] OfferId: %s"), *OfferId);
						}
					}
					else
					{
						UE_LOG_ONLINE(Error, TEXT("[FOnlineStoreEOSV2::HandleEcomExec] QueryOffers error (%s)"), *ErrorStr);
					}
				}));
		return true;
	}
	return false;
}

#endif