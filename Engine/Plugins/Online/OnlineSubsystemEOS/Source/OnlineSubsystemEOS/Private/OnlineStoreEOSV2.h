// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Online/CoreOnline.h"
#include "Interfaces/OnlineStoreInterfaceV2.h"
#include "OnlineSubsystemEOSTypes.h"

#if WITH_EOS_SDK
#include "eos_ecom_types.h"

class UWorld;

/**
 * Retrieve offers from store. Offers can be checked out using the OnlinePurchase interface
 */
class FOnlineStoreEOSV2 :
	public IOnlineStoreV2,
	public TSharedFromThis<FOnlineStoreEOSV2, ESPMode::ThreadSafe>
{
public:
	virtual ~FOnlineStoreEOSV2() = default;
	
	virtual void QueryCategories(const FUniqueNetId& UserId, const FOnQueryOnlineStoreCategoriesComplete& Delegate) override;
	virtual void GetCategories(TArray<FOnlineStoreCategory>& OutCategories) const override;
	virtual void QueryOffersByFilter(const FUniqueNetId& UserId, const FOnlineStoreFilter& Filter, const FOnQueryOnlineStoreOffersComplete& Delegate) override;
	virtual void QueryOffersById(const FUniqueNetId& UserId, const TArray<FUniqueOfferId>& OfferIds, const FOnQueryOnlineStoreOffersComplete& Delegate) override;
	virtual void GetOffers(TArray<FOnlineStoreOfferRef>& OutOffers) const override;
	virtual TSharedPtr<FOnlineStoreOffer> GetOffer(const FUniqueOfferId& OfferId) const override;
	
	FOnlineStoreEOSV2(FOnlineSubsystemEOS* InSubsystem);
	
	bool HandleEcomExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);

private:
	/** Default constructor disabled */
	FOnlineStoreEOSV2() = delete;

	/** Retrieve Epic Games Store offers */
	void QueryOffers(const FUniqueNetId& UserId, const FOnQueryOnlineStoreOffersComplete& Delegate);

	/** Reference to the main EOS subsystem */
	FOnlineSubsystemEOS* EOSSubsystem;

	/** The set of offers for this title */
	TArray<FOnlineStoreOfferRef> CachedOffers;
	/** List of offer ids for this title */
	TArray<FUniqueOfferId> CachedOfferIds;
};

typedef TSharedPtr<FOnlineStoreEOSV2, ESPMode::ThreadSafe> FOnlineStoreEOSV2Ptr;
typedef TWeakPtr<FOnlineStoreEOSV2, ESPMode::ThreadSafe> FOnlineStoreEOSV2WeakPtr;

#endif
