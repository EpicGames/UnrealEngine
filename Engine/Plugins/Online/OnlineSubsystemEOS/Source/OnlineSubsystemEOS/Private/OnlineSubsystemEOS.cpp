// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemEOS.h"
#include "OnlineSubsystemEOSPrivate.h"
#include "OnlineSubsystemUtils.h"
#include "UserManagerEOS.h"
#include "OnlineSessionEOS.h"
#include "OnlineStatsEOS.h"
#include "OnlineLeaderboardsEOS.h"
#include "OnlineAchievementsEOS.h"
#include "OnlineTitleFileEOS.h"
#include "OnlineUserCloudEOS.h"
#include "OnlineStoreEOS.h"
#include "OnlinePlayerSanctionEOS.h"
#include "OnlinePlayerReportEOS.h"
#include "OnlineEntitlementsEOS.h"
#include "OnlinePurchaseEOS.h"
#include "OnlineStoreEOSV2.h"
#include "EOSSettings.h"
#include "EOSShared.h"
#include "EOSVoiceChat.h"
#include "IEOSSDKManager.h"
#include "SocketSubsystemEOSUtils_OnlineSubsystemEOS.h"

#include "Features/IModularFeatures.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Fork.h"
#include "Misc/NetworkVersion.h"

#if WITH_EOS_SDK

// Missing defines
#define EOS_ENCRYPTION_KEY_MAX_LENGTH 64
#define EOS_ENCRYPTION_KEY_MAX_BUFFER_LEN (EOS_ENCRYPTION_KEY_MAX_LENGTH + 1)

#if WITH_EOSVOICECHAT

#include "EOSVoiceChatFactory.h"
#include "EOSVoiceChatUser.h"
#include "VoiceChatResult.h"
#include "VoiceChatErrors.h"

/** Class that blocks login/logout for the OSS EOS managed IVoiceChatUser interfaces. */
class FOnlineSubsystemEOSVoiceChatUserWrapper : public IVoiceChatUser
{
public:
	FOnlineSubsystemEOSVoiceChatUserWrapper(FEOSVoiceChatUser& InVoiceChatUser) : VoiceChatUser(InVoiceChatUser) {}
	~FOnlineSubsystemEOSVoiceChatUserWrapper() = default;

	// ~Begin IVoiceChatUser
	virtual void SetSetting(const FString& Name, const FString& Value) override { VoiceChatUser.SetSetting(Name, Value); }
	virtual FString GetSetting(const FString& Name) override { return VoiceChatUser.GetSetting(Name); }
	virtual void SetAudioInputVolume(float Volume) override { VoiceChatUser.SetAudioInputVolume(Volume); }
	virtual void SetAudioOutputVolume(float Volume) override { VoiceChatUser.SetAudioOutputVolume(Volume); }
	virtual float GetAudioInputVolume() const override { return VoiceChatUser.GetAudioInputVolume(); }
	virtual float GetAudioOutputVolume() const override { return VoiceChatUser.GetAudioOutputVolume(); }
	virtual void SetAudioInputDeviceMuted(bool bIsMuted) override { VoiceChatUser.SetAudioInputDeviceMuted(bIsMuted); }
	virtual void SetAudioOutputDeviceMuted(bool bIsMuted) override { VoiceChatUser.SetAudioOutputDeviceMuted(bIsMuted); }
	virtual bool GetAudioInputDeviceMuted() const override { return VoiceChatUser.GetAudioInputDeviceMuted(); }
	virtual bool GetAudioOutputDeviceMuted() const override { return VoiceChatUser.GetAudioOutputDeviceMuted(); }
	virtual TArray<FVoiceChatDeviceInfo> GetAvailableInputDeviceInfos() const override { return VoiceChatUser.GetAvailableInputDeviceInfos(); }
	virtual TArray<FVoiceChatDeviceInfo> GetAvailableOutputDeviceInfos() const override { return VoiceChatUser.GetAvailableOutputDeviceInfos(); }
	virtual FOnVoiceChatAvailableAudioDevicesChangedDelegate& OnVoiceChatAvailableAudioDevicesChanged() override { return VoiceChatUser.OnVoiceChatAvailableAudioDevicesChanged(); }
	virtual void SetInputDeviceId(const FString& InputDeviceId) override { VoiceChatUser.SetInputDeviceId(InputDeviceId); }
	virtual void SetOutputDeviceId(const FString& OutputDeviceId) override { VoiceChatUser.SetOutputDeviceId(OutputDeviceId); }
	virtual FVoiceChatDeviceInfo GetInputDeviceInfo() const override { return VoiceChatUser.GetInputDeviceInfo(); }
	virtual FVoiceChatDeviceInfo GetOutputDeviceInfo() const override { return VoiceChatUser.GetOutputDeviceInfo(); }
	virtual FVoiceChatDeviceInfo GetDefaultInputDeviceInfo() const override { return VoiceChatUser.GetDefaultInputDeviceInfo(); }
	virtual FVoiceChatDeviceInfo GetDefaultOutputDeviceInfo() const override { return VoiceChatUser.GetDefaultOutputDeviceInfo(); }
	virtual void Login(FPlatformUserId PlatformId, const FString& PlayerName, const FString& Credentials, const FOnVoiceChatLoginCompleteDelegate& Delegate) override
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: IVoiceChatUser::Login called on OSS EOS managed VoiceChatUser interface."));
		checkNoEntry();
		Delegate.ExecuteIfBound(PlayerName, VoiceChat::Errors::NotPermitted());
	}
	virtual void Logout(const FOnVoiceChatLogoutCompleteDelegate& Delegate) override
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: IVoiceChatUser::Logout called on OSS EOS managed VoiceChatUser interface."));
		checkNoEntry();
		Delegate.ExecuteIfBound(VoiceChatUser.GetLoggedInPlayerName(), VoiceChat::Errors::NotPermitted());
	}
	virtual bool IsLoggingIn() const override { return VoiceChatUser.IsLoggingIn(); }
	virtual bool IsLoggedIn() const override { return VoiceChatUser.IsLoggedIn(); }
	virtual FOnVoiceChatLoggedInDelegate& OnVoiceChatLoggedIn() override { return VoiceChatUser.OnVoiceChatLoggedIn(); }
	virtual FOnVoiceChatLoggedOutDelegate& OnVoiceChatLoggedOut() override { return VoiceChatUser.OnVoiceChatLoggedOut(); }
	virtual FString GetLoggedInPlayerName() const override { return VoiceChatUser.GetLoggedInPlayerName(); }
	virtual void BlockPlayers(const TArray<FString>& PlayerNames) override { VoiceChatUser.BlockPlayers(PlayerNames); }
	virtual void UnblockPlayers(const TArray<FString>& PlayerNames) override { return VoiceChatUser.UnblockPlayers(PlayerNames); }
	virtual void JoinChannel(const FString& ChannelName, const FString& ChannelCredentials, EVoiceChatChannelType ChannelType, const FOnVoiceChatChannelJoinCompleteDelegate& Delegate, TOptional<FVoiceChatChannel3dProperties> Channel3dProperties = TOptional<FVoiceChatChannel3dProperties>()) override { VoiceChatUser.JoinChannel(ChannelName, ChannelCredentials, ChannelType, Delegate, Channel3dProperties); }
	virtual void LeaveChannel(const FString& ChannelName, const FOnVoiceChatChannelLeaveCompleteDelegate& Delegate) override { VoiceChatUser.LeaveChannel(ChannelName, Delegate); }
	virtual FOnVoiceChatChannelJoinedDelegate& OnVoiceChatChannelJoined() override { return VoiceChatUser.OnVoiceChatChannelJoined(); }
	virtual FOnVoiceChatChannelExitedDelegate& OnVoiceChatChannelExited() override { return VoiceChatUser.OnVoiceChatChannelExited(); }
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual FOnVoiceChatCallStatsUpdatedDelegate& OnVoiceChatCallStatsUpdated() override { return VoiceChatUser.OnVoiceChatCallStatsUpdated(); }
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	virtual FOnVoiceChatCallStatsUpdatedDelegate2& OnVoiceChatCallStatsUpdated2() override { return VoiceChatUser.OnVoiceChatCallStatsUpdated2(); }
	virtual TOptional<FVoiceChatCallStats> GetChannelCallStats(const FString& ChannelName) const override { return VoiceChatUser.GetChannelCallStats(ChannelName); }
	virtual void Set3DPosition(const FString& ChannelName, const FVector& Position) override { VoiceChatUser.Set3DPosition(ChannelName, Position); }
	virtual TArray<FString> GetChannels() const override { return VoiceChatUser.GetChannels(); }
	virtual TArray<FString> GetPlayersInChannel(const FString& ChannelName) const override { return VoiceChatUser.GetPlayersInChannel(ChannelName); }
	virtual EVoiceChatChannelType GetChannelType(const FString& ChannelName) const override { return VoiceChatUser.GetChannelType(ChannelName); }
	virtual FOnVoiceChatPlayerAddedDelegate& OnVoiceChatPlayerAdded() override { return VoiceChatUser.OnVoiceChatPlayerAdded(); }
	virtual FOnVoiceChatPlayerRemovedDelegate& OnVoiceChatPlayerRemoved() override { return VoiceChatUser.OnVoiceChatPlayerRemoved(); }
	virtual bool IsPlayerTalking(const FString& PlayerName) const override { return VoiceChatUser.IsPlayerTalking(PlayerName); }
	virtual FOnVoiceChatPlayerTalkingUpdatedDelegate& OnVoiceChatPlayerTalkingUpdated() override { return VoiceChatUser.OnVoiceChatPlayerTalkingUpdated(); }
	virtual void SetPlayerMuted(const FString& PlayerName, bool bMuted) override { VoiceChatUser.SetPlayerMuted(PlayerName, bMuted); }
	virtual bool IsPlayerMuted(const FString& PlayerName) const override { return VoiceChatUser.IsPlayerMuted(PlayerName); }
	virtual void SetChannelPlayerMuted(const FString& ChannelName, const FString& PlayerName, bool bMuted) override { VoiceChatUser.SetChannelPlayerMuted(ChannelName, PlayerName, bMuted); }
	virtual bool IsChannelPlayerMuted(const FString& ChannelName, const FString& PlayerName) const override { return VoiceChatUser.IsChannelPlayerMuted(ChannelName, PlayerName); }
	virtual FOnVoiceChatPlayerMuteUpdatedDelegate& OnVoiceChatPlayerMuteUpdated() override { return VoiceChatUser.OnVoiceChatPlayerMuteUpdated(); }
	virtual void SetPlayerVolume(const FString& PlayerName, float Volume) override { VoiceChatUser.SetPlayerVolume(PlayerName, Volume); }
	virtual float GetPlayerVolume(const FString& PlayerName) const override { return VoiceChatUser.GetPlayerVolume(PlayerName); }
	virtual FOnVoiceChatPlayerVolumeUpdatedDelegate& OnVoiceChatPlayerVolumeUpdated() override { return VoiceChatUser.OnVoiceChatPlayerVolumeUpdated(); }
	virtual void TransmitToAllChannels() override { VoiceChatUser.TransmitToAllChannels(); }
	virtual void TransmitToNoChannels() override { VoiceChatUser.TransmitToNoChannels(); }
	virtual void TransmitToSpecificChannels(const TSet<FString>& ChannelNames) override { VoiceChatUser.TransmitToSpecificChannels(ChannelNames); }
	virtual EVoiceChatTransmitMode GetTransmitMode() const override { return VoiceChatUser.GetTransmitMode(); }
	virtual TSet<FString> GetTransmitChannels() const override { return VoiceChatUser.GetTransmitChannels(); }
	virtual FDelegateHandle StartRecording(const FOnVoiceChatRecordSamplesAvailableDelegate::FDelegate& Delegate) override { return VoiceChatUser.StartRecording(Delegate); }
	virtual void StopRecording(FDelegateHandle Handle) override { VoiceChatUser.StopRecording(Handle); }
	virtual FDelegateHandle RegisterOnVoiceChatAfterCaptureAudioReadDelegate(const FOnVoiceChatAfterCaptureAudioReadDelegate2::FDelegate& Delegate) override { return VoiceChatUser.RegisterOnVoiceChatAfterCaptureAudioReadDelegate(Delegate); }
	virtual void UnregisterOnVoiceChatAfterCaptureAudioReadDelegate(FDelegateHandle Handle) override { VoiceChatUser.UnregisterOnVoiceChatAfterCaptureAudioReadDelegate(Handle); }
	virtual FDelegateHandle RegisterOnVoiceChatBeforeCaptureAudioSentDelegate(const FOnVoiceChatBeforeCaptureAudioSentDelegate2::FDelegate& Delegate) override { return VoiceChatUser.RegisterOnVoiceChatBeforeCaptureAudioSentDelegate(Delegate); }
	virtual void UnregisterOnVoiceChatBeforeCaptureAudioSentDelegate(FDelegateHandle Handle) override { VoiceChatUser.UnregisterOnVoiceChatBeforeCaptureAudioSentDelegate(Handle); }
	virtual FDelegateHandle RegisterOnVoiceChatBeforeRecvMixedAudioRenderedDelegate(const FOnVoiceChatBeforeRecvAudioRenderedDelegate::FDelegate& Delegate) override { return VoiceChatUser.RegisterOnVoiceChatBeforeRecvMixedAudioRenderedDelegate(Delegate); }
	virtual void UnregisterOnVoiceChatBeforeRecvMixedAudioRenderedDelegate(FDelegateHandle Handle) override { VoiceChatUser.UnregisterOnVoiceChatBeforeRecvMixedAudioRenderedDelegate(Handle); }
	virtual FDelegateHandle RegisterOnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate(const FOnVoiceChatBeforeRecvAudioRenderedDelegate::FDelegate& Delegate) override { return VoiceChatUser.RegisterOnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate(Delegate); }
	virtual void UnregisterOnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate(FDelegateHandle Handle) override { VoiceChatUser.UnregisterOnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate(Handle); }
	virtual FString InsecureGetLoginToken(const FString& PlayerName) override { return VoiceChatUser.InsecureGetLoginToken(PlayerName); }
	virtual FString InsecureGetJoinToken(const FString& ChannelName, EVoiceChatChannelType ChannelType, TOptional<FVoiceChatChannel3dProperties> Channel3dProperties = TOptional<FVoiceChatChannel3dProperties>()) override { return VoiceChatUser.InsecureGetJoinToken(ChannelName, ChannelType, Channel3dProperties); }
	// ~End IVoiceChatUser

	FEOSVoiceChatUser& VoiceChatUser;
};

#endif // WITH_EOSVOICECHAT

FPlatformEOSHelpersPtr FOnlineSubsystemEOS::EOSHelpersPtr;

void FOnlineSubsystemEOS::ModuleInit()
{
	LLM_SCOPE(ELLMTag::RealTimeCommunications);

	EOSHelpersPtr = MakeShareable(new FPlatformEOSHelpers());

	const FName EOSSharedModuleName = TEXT("EOSShared");
	if (!FModuleManager::Get().IsModuleLoaded(EOSSharedModuleName))
	{
		FModuleManager::Get().LoadModuleChecked(EOSSharedModuleName);
	}
	IEOSSDKManager* EOSSDKManager = IEOSSDKManager::Get();
	if (!EOSSDKManager)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: Missing IEOSSDKManager modular feature."));
		return;
	}

	// If a fork is requested, we need to wait for post-fork to init the SDK
	if (!FForkProcessHelper::IsForkRequested() || FForkProcessHelper::IsForkedChildProcess())
	{
		if (!EOSSDKManager->IsInitialized())
		{
			UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: EOSSDK not initialized"));
			return;
		}
	}
}

void FOnlineSubsystemEOS::ModuleShutdown()
{
#define DESTRUCT_INTERFACE(Interface) \
	if (Interface.IsValid()) \
	{ \
		ensure(Interface.IsUnique()); \
		Interface = nullptr; \
	}

	DESTRUCT_INTERFACE(EOSHelpersPtr);

#undef DESTRUCT_INTERFACE
}

/** Common method for creating the EOS platform */
bool FOnlineSubsystemEOS::PlatformCreate()
{
	const FEOSSettings& EOSSettings = UEOSSettings::GetSettings();

	if (EOSSettings.bUseNamedPlatformConfig)
	{
		EOSPlatformHandle = EOSSDKManager->CreatePlatform(EOSSettings.PlatformConfigName, InstanceName == FOnlineSubsystemImpl::DefaultInstanceName ? NAME_None : InstanceName);
		UE_CLOG_ONLINE(!EOSPlatformHandle.IsValid(), Error, TEXT("FOnlineSubsystemEOS::PlatformCreate() PlatformConfigName=%s failed"),
			*EOSSettings.PlatformConfigName);
		return EOSPlatformHandle.IsValid();
	}

	FEOSArtifactSettings ArtifactSettings;
	if (!UEOSSettings::GetSelectedArtifactSettings(ArtifactSettings))
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS::PlatformCreate() GetSelectedArtifactSettings failed"));
		return false;
	}

	const auto ClientIdUtf8 = StringCast<UTF8CHAR>(*ArtifactSettings.ClientId);
	const auto ClientSecretUtf8 = StringCast<UTF8CHAR>(*ArtifactSettings.ClientSecret);
	const auto ProductIdUtf8 = StringCast<UTF8CHAR>(*ArtifactSettings.ProductId);
	const auto SandboxIdUtf8 = StringCast<UTF8CHAR>(*ArtifactSettings.SandboxId);
	const auto DeploymentIdUtf8 = StringCast<UTF8CHAR>(*ArtifactSettings.DeploymentId);
	const auto EncryptionKeyUtf8 = StringCast<UTF8CHAR>(*ArtifactSettings.EncryptionKey);

	// Create platform instance
	EOS_Platform_Options PlatformOptions = {};
	PlatformOptions.ApiVersion = 14;
	UE_EOS_CHECK_API_MISMATCH(EOS_PLATFORM_OPTIONS_API_LATEST, 15);
	PlatformOptions.ClientCredentials.ClientId = reinterpret_cast<const char*>(ArtifactSettings.ClientId.IsEmpty() ? nullptr : (const char*)ClientIdUtf8.Get());
	PlatformOptions.ClientCredentials.ClientSecret = ArtifactSettings.ClientSecret.IsEmpty() ? nullptr : (const char*)ClientSecretUtf8.Get();
	PlatformOptions.ProductId = ArtifactSettings.ProductId.IsEmpty() ? nullptr : (const char*)ProductIdUtf8.Get();
	PlatformOptions.SandboxId = ArtifactSettings.SandboxId.IsEmpty() ? nullptr : (const char*)SandboxIdUtf8.Get();
	PlatformOptions.DeploymentId = ArtifactSettings.DeploymentId.IsEmpty() ? nullptr : (const char*)DeploymentIdUtf8.Get();
	PlatformOptions.EncryptionKey = ArtifactSettings.EncryptionKey.IsEmpty() ? nullptr : (const char*)EncryptionKeyUtf8.Get();
	PlatformOptions.bIsServer = IsRunningDedicatedServer() ? EOS_TRUE : EOS_FALSE;
	PlatformOptions.Reserved = nullptr;
	PlatformOptions.TaskNetworkTimeoutSeconds = nullptr;

	uint64 OverlayFlags = 0;
	if (!EOSSettings.bEnableOverlay)
	{
		OverlayFlags |= EOS_PF_DISABLE_OVERLAY;
	}
	if (!EOSSettings.bEnableSocialOverlay)
	{
		OverlayFlags |= EOS_PF_DISABLE_SOCIAL_OVERLAY;
	}
#if WITH_EDITOR
	if (GIsEditor && EOSSettings.bEnableEditorOverlay)
	{
		OverlayFlags |= EOS_PF_LOADING_IN_EDITOR;
	}
#endif

	// Don't allow the overlay to be used in the editor when running PIE.
	const bool bEditorOverlayAllowed = EOSSettings.bEnableEditorOverlay && InstanceName == FOnlineSubsystemImpl::DefaultInstanceName;
	const bool bOverlayAllowed = IsRunningGame() || bEditorOverlayAllowed;

	PlatformOptions.Flags = bOverlayAllowed ? OverlayFlags : EOS_PF_DISABLE_OVERLAY;

	// Make the cache directory be in the user's writable area
	FString CacheDir;
	
	const FString CacheDirBase = EOSSDKManager->GetCacheDirBase();

	if (!CacheDirBase.IsEmpty())
	{
		CacheDir = CacheDirBase / ArtifactSettings.ArtifactName / EOSSettings.CacheDir;
	}
	const auto CacheDirUtf8 = StringCast<UTF8CHAR>(*CacheDir);
	PlatformOptions.CacheDirectory = CacheDir.IsEmpty() ? nullptr : (const char*)CacheDirUtf8.Get();

#if WITH_EOS_RTC
	EOS_Platform_RTCOptions RtcOptions = { 0 };
	RtcOptions.ApiVersion = 3;
	UE_EOS_CHECK_API_MISMATCH(EOS_PLATFORM_RTCOPTIONS_API_LATEST, 3);
	RtcOptions.PlatformSpecificOptions = nullptr;
	RtcOptions.BackgroundMode = EOSSettings.RTCBackgroundMode;
	RtcOptions.Reserved = nullptr;
	PlatformOptions.RTCOptions = EOSSettings.bUseEOSRTC ? &RtcOptions : nullptr;
#endif

	EOSPlatformHandle = EOSHelpersPtr->CreatePlatform(PlatformOptions);
	if (EOSPlatformHandle == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS::PlatformCreate() failed to init EOS platform"));
		return false;
	}
	
	return true;
}

bool FOnlineSubsystemEOS::Init()
{
	bool bShouldEnforceBeingLaunchedByEGS = false;
	GConfig->GetBool(TEXT("/Script/OnlineSubsystemEOS.EOSSettings"), TEXT("bShouldEnforceBeingLaunchedByEGS"), bShouldEnforceBeingLaunchedByEGS, GEngineIni);
	if (bShouldEnforceBeingLaunchedByEGS)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: Support for bShouldEnforceBeingLaunchedByEGS has been removed, please delete this config entry and instead set bUseLauncherChecks=true in your .Target.cs file(s)"));
		return false;
	}

	EOSSDKManager = IEOSSDKManager::Get();
	if (!EOSSDKManager)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: failed to get EOSSDKManager interface"));
		return false;
	}

	if (!EOSSDKManager->IsInitialized())
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: EOSSDK not initialized"));
		return false;
	}

	if (!PlatformCreate())
	{
		return false;
	}

	// Get handles for later use
	AuthHandle = EOS_Platform_GetAuthInterface(*EOSPlatformHandle);
	if (AuthHandle == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: failed to init EOS platform, couldn't get auth handle"));
		return false;
	}
	UserInfoHandle = EOS_Platform_GetUserInfoInterface(*EOSPlatformHandle);
	if (UserInfoHandle == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: failed to init EOS platform, couldn't get user info handle"));
		return false;
	}
	UIHandle = EOS_Platform_GetUIInterface(*EOSPlatformHandle);
	if (UIHandle == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: failed to init EOS platform, couldn't get UI handle"));
		return false;
	}
	FriendsHandle = EOS_Platform_GetFriendsInterface(*EOSPlatformHandle);
	if (FriendsHandle == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: failed to init EOS platform, couldn't get friends handle"));
		return false;
	}
	PresenceHandle = EOS_Platform_GetPresenceInterface(*EOSPlatformHandle);
	if (PresenceHandle == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: failed to init EOS platform, couldn't get presence handle"));
		return false;
	}
	ConnectHandle = EOS_Platform_GetConnectInterface(*EOSPlatformHandle);
	if (ConnectHandle == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: failed to init EOS platform, couldn't get connect handle"));
		return false;
	}
	SessionsHandle = EOS_Platform_GetSessionsInterface(*EOSPlatformHandle);
	if (SessionsHandle == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: failed to init EOS platform, couldn't get sessions handle"));
		return false;
	}
	StatsHandle = EOS_Platform_GetStatsInterface(*EOSPlatformHandle);
	if (StatsHandle == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: failed to init EOS platform, couldn't get stats handle"));
		return false;
	}
	LeaderboardsHandle = EOS_Platform_GetLeaderboardsInterface(*EOSPlatformHandle);
	if (LeaderboardsHandle == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: failed to init EOS platform, couldn't get leaderboards handle"));
		return false;
	}
	MetricsHandle = EOS_Platform_GetMetricsInterface(*EOSPlatformHandle);
	if (MetricsHandle == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: failed to init EOS platform, couldn't get metrics handle"));
		return false;
	}
	AchievementsHandle = EOS_Platform_GetAchievementsInterface(*EOSPlatformHandle);
	if (AchievementsHandle == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: failed to init EOS platform, couldn't get achievements handle"));
		return false;
	}
	EcomHandle = EOS_Platform_GetEcomInterface(*EOSPlatformHandle);
	if (EcomHandle == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: failed to init EOS platform, couldn't get ecom handle"));
		return false;
	}
	PlayerSanctionHandle = EOS_Platform_GetSanctionsInterface(*EOSPlatformHandle);
	if (PlayerSanctionHandle == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: failed to init EOS platform, couldn't get player sanction handle"));
		return false;
	}
	PlayerReportHandle = EOS_Platform_GetReportsInterface(*EOSPlatformHandle);
	if (PlayerReportHandle == nullptr)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: failed to init EOS platform, couldn't get player report handle"));
		return false;
	}

	// We set the product id
	FString EncryptionKey, ArtifactName;

	const FEOSSettings& EOSSettings = UEOSSettings::GetSettings();
	if (EOSSettings.bUseNamedPlatformConfig)
	{
		if (const FEOSSDKPlatformConfig* PlatformConfig = EOSSDKManager->GetPlatformConfig(EOSSettings.PlatformConfigName))
		{
			ProductId = PlatformConfig->ProductId;
			EncryptionKey = PlatformConfig->EncryptionKey;
			ArtifactName = PlatformConfig->Name;
		}
		else
		{
			// it souldn't be possible to get here if PlatformCreate succeeded
			checkNoEntry();
			UE_LOG_ONLINE(Error, TEXT("[FOnlineSubsystemEOS::Init] GetPlatformConfig failed"));
			return false;
		}
	}
	else
	{
		FEOSArtifactSettings ArtifactSettings;
		if (UEOSSettings::GetSelectedArtifactSettings(ArtifactSettings))
		{
			ProductId = ArtifactSettings.ProductId;
			EncryptionKey = ArtifactSettings.EncryptionKey;
			ArtifactName = ArtifactSettings.ArtifactName;
		}
		else
		{
			// it souldn't be possible to get here if PlatformCreate succeeded
			checkNoEntry();
			UE_LOG_ONLINE(Error, TEXT("[FOnlineSubsystemEOS::Init] GetSelectedArtifactSettings failed"));
			return false;
		}
	}

	TitleStorageHandle = EOS_Platform_GetTitleStorageInterface(*EOSPlatformHandle);
	if (TitleStorageHandle)
	{
		TitleFileInterfacePtr = MakeShared<FOnlineTitleFileEOS>(this);
	}
	else if (EncryptionKey.IsEmpty())
	{
		UE_LOG_ONLINE(Verbose, TEXT("FOnlineSubsystemEOS: ArtifactName=[%s] EncryptionKey unset, TitleFile interface unavailable."), *ArtifactName);
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: failed to init EOS platform, couldn't get title storage handle"));
		return false;
	}

	PlayerDataStorageHandle = EOS_Platform_GetPlayerDataStorageInterface(*EOSPlatformHandle);
	if (PlayerDataStorageHandle)
	{
		UserCloudInterfacePtr = MakeShared<FOnlineUserCloudEOS>(this);
	}
	else if (EncryptionKey.IsEmpty())
	{
		UE_LOG_ONLINE(Verbose, TEXT("FOnlineSubsystemEOS: ArtifactName=[%s] EncryptionKey unset, UserCloud interface unavailable."), *ArtifactName);
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemEOS: failed to init EOS platform, couldn't get player data storage handle"));
		return false;
	}

	SocketSubsystem = MakeShareable(new FSocketSubsystemEOS(EOSPlatformHandle, MakeShareable(new FSocketSubsystemEOSUtils_OnlineSubsystemEOS(*this))));
	check(SocketSubsystem);
	FString ErrorMessage;
	if (!SocketSubsystem->Init(ErrorMessage))
	{
		UE_LOG_ONLINE(Warning, TEXT("[FOnlineSubsystemEOS::Init] Unable to initialize Socket Subsystem. Error=[%s]"), *ErrorMessage);
	}

	UserManager = MakeShared<FUserManagerEOS>(this);
	UserManager->Init();
	SessionInterfacePtr = MakeShared<FOnlineSessionEOS>(this);
	SessionInterfacePtr->Init();
	StatsInterfacePtr = MakeShared<FOnlineStatsEOS>(this);
	LeaderboardsInterfacePtr = MakeShared<FOnlineLeaderboardsEOS>(this);
	AchievementsInterfacePtr = MakeShared<FOnlineAchievementsEOS>(this);
	PlayerSanctionEOSPtr = MakeShared<FOnlinePlayerSanctionEOS>(this);
	PlayerReportInterfacePtr = MakeShared<FOnlinePlayerReportEOS>(this);

	FEOSSettings Settings = UEOSSettings::GetSettings();
	if (Settings.bUseNewEcomFlow)
	{
		EntitlementsInterfacePtr = MakeShared<FOnlineEntitlementsEOS>(this);
		PurchaseInterfacePtr = MakeShared<FOnlinePurchaseEOS>(this);
		StoreInterfaceV2Ptr = MakeShared<FOnlineStoreEOSV2>(this);
	}
	else
	{
		StoreInterfacePtr = MakeShared<FOnlineStoreEOS>(this);
		UE_LOG_ONLINE(Warning, TEXT("[FOnlineSubsystemEOS::Init] bUseNewEcomFlow set to false. Note the old ecom flow will be deprecated in a future version"));
	}

	// We initialized ok so we can tick
	StartTicker();

	return true;
}

bool FOnlineSubsystemEOS::Shutdown()
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineSubsystemEOS::Shutdown()"));

	// EOS-22677 workaround: Make sure tick is called at least once before shutting down.
	if (EOSPlatformHandle)
	{
		EOS_Platform_Tick(*EOSPlatformHandle);
	}

	StopTicker();

	if (SocketSubsystem)
	{
		SocketSubsystem->Shutdown();
		SocketSubsystem = nullptr;
	}

	// Release our ref to the interfaces. May still exist since they can be aggregated
	UserManager = nullptr;
	SessionInterfacePtr = nullptr;
	StatsInterfacePtr = nullptr;
	LeaderboardsInterfacePtr = nullptr;
	AchievementsInterfacePtr = nullptr;
	TitleFileInterfacePtr = nullptr;
	UserCloudInterfacePtr = nullptr;
	EntitlementsInterfacePtr = nullptr;
	PurchaseInterfacePtr = nullptr;
	StoreInterfaceV2Ptr = nullptr;
	StoreInterfacePtr = nullptr;

#if WITH_EOSVOICECHAT
	for (TPair<FUniqueNetIdRef, FOnlineSubsystemEOSVoiceChatUserWrapperRef>& Pair : LocalVoiceChatUsers)
	{
		FOnlineSubsystemEOSVoiceChatUserWrapperRef& VoiceChatUserWrapper = Pair.Value;
		VoiceChatInterface->ReleaseUser(&VoiceChatUserWrapper->VoiceChatUser);
	}
	LocalVoiceChatUsers.Reset();
	VoiceChatInterface = nullptr;
#endif // WITH_EOSVOICECHAT

	EOSPlatformHandle = nullptr;

	return FOnlineSubsystemImpl::Shutdown();
}

bool FOnlineSubsystemEOS::Tick(float DeltaTime)
{
	if (!bTickerStarted)
	{
		return true;
	}

	SessionInterfacePtr->Tick(DeltaTime);
	FOnlineSubsystemImpl::Tick(DeltaTime);

	return true;
}

bool FOnlineSubsystemEOS::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FOnlineSubsystemImpl::Exec(InWorld, Cmd, Ar))
	{
		return true;
	}

	bool bWasHandled = false;
	if (UserManager != nullptr && FParse::Command(&Cmd, TEXT("FRIENDS"))) /* ONLINE FRIENDS ... */
	{
		bWasHandled = UserManager->HandleFriendsExec(InWorld, Cmd, Ar);
	}
	else if (StoreInterfacePtr != nullptr && FParse::Command(&Cmd, TEXT("ECOM"))) /* ONLINE ECOM ... only ran if bUseNewEcomFlow is false */
	{
		bWasHandled = StoreInterfacePtr->HandleEcomExec(InWorld, Cmd, Ar);
	}
	else if (StoreInterfaceV2Ptr != nullptr && FParse::Command(&Cmd, TEXT("ECOM"))) /* ONLINE ECOM ... only ran if bUseNewEcomFlow is true */
	{
		bWasHandled = StoreInterfaceV2Ptr->HandleEcomExec(InWorld, Cmd, Ar);
	}
	else if (PurchaseInterfacePtr != nullptr && FParse::Command(&Cmd, TEXT("PURCHASE"))) /* ONLINE PURCHASE ... only ran if bUseNewEcomFlow is true */
	{
		bWasHandled = PurchaseInterfacePtr->HandlePurchaseExec(InWorld, Cmd, Ar);
	}
	else if (EntitlementsInterfacePtr != nullptr && FParse::Command(&Cmd, TEXT("ENTITLEMENTS"))) /* ONLINE ENTITLEMENTS ... only ran if bUseNewEcomFlow is true */
	{
		bWasHandled = EntitlementsInterfacePtr->HandleEntitlementsExec(InWorld, Cmd, Ar);
	}
	else if (LeaderboardsInterfacePtr != nullptr && FParse::Command(&Cmd, TEXT("LEADERBOARDS"))) /* ONLINE LEADERBOARDS ... */
	{
		bWasHandled = LeaderboardsInterfacePtr->HandleLeaderboardsExec(InWorld, Cmd, Ar);
	}
	else if (SessionInterfacePtr != nullptr && FParse::Command(&Cmd, TEXT("SESSION"))) /* ONLINE SESSION ... */
	{
		bWasHandled = SessionInterfacePtr->HandleSessionExec(InWorld, Cmd, Ar);
	}
	else if (TitleFileInterfacePtr != nullptr && FParse::Command(&Cmd, TEXT("TITLEFILE"))) /* ONLINE TITLEFILE ... */
	{
		bWasHandled = TitleFileInterfacePtr->HandleTitleFileExec(InWorld, Cmd, Ar);
	}
	else if (UserCloudInterfacePtr != nullptr && FParse::Command(&Cmd, TEXT("USERCLOUD"))) /* ONLINE USERCLOUD ... */
	{
		bWasHandled = UserCloudInterfacePtr->HandleUserCloudExec(InWorld, Cmd, Ar);
	}
	else
	{
		bWasHandled = false;
	}

	return bWasHandled;
}

void FOnlineSubsystemEOS::ReloadConfigs(const TSet<FString>& ConfigSections)
{
	UE_LOG_ONLINE(Verbose, TEXT("FOnlineSubsystemEOS::ReloadConfigs"));

	// There is currently no granular reloading, so just restart the subsystem to pick up new config.
	const bool bWasInitialized = EOSPlatformHandle != nullptr;
	const bool bConfigChanged = ConfigSections.Find(GetDefault<UEOSSettings>()->GetClass()->GetPathName()) != nullptr;
	const bool bRestartRequired = bWasInitialized && bConfigChanged;

	if (bRestartRequired)
	{
		UE_LOG_ONLINE(Verbose, TEXT("FOnlineSubsystemEOS::ReloadConfigs: Restarting subsystem to pick up changes."));
		PreUnload();
		Shutdown();
	}

	// Notify user code so that overrides may be applied.
	TriggerOnConfigChangedDelegates(ConfigSections);

	// Reload config objects.
	if (bConfigChanged)
	{
		GetMutableDefault<UEOSSettings>()->ReloadConfig();
	}

	if (bRestartRequired)
	{
		Init();
	}
}

FString FOnlineSubsystemEOS::GetAppId() const
{
	return TEXT("");
}

FText FOnlineSubsystemEOS::GetOnlineServiceName() const
{
	return NSLOCTEXT("OnlineSubsystemEOS", "OnlineServiceName", "EOS");
}

FOnlineSubsystemEOS::FOnlineSubsystemEOS(FName InInstanceName) :
	IOnlineSubsystemEOS(EOS_SUBSYSTEM, InInstanceName)
	, EOSSDKManager(nullptr)
	, AuthHandle(nullptr)
	, UIHandle(nullptr)
	, FriendsHandle(nullptr)
	, UserInfoHandle(nullptr)
	, PresenceHandle(nullptr)
	, ConnectHandle(nullptr)
	, SessionsHandle(nullptr)
	, StatsHandle(nullptr)
	, LeaderboardsHandle(nullptr)
	, MetricsHandle(nullptr)
	, AchievementsHandle(nullptr)
	, EcomHandle(nullptr)
	, TitleStorageHandle(nullptr)
	, PlayerDataStorageHandle(nullptr)
	, UserManager(nullptr)
	, SessionInterfacePtr(nullptr)
	, LeaderboardsInterfacePtr(nullptr)
	, AchievementsInterfacePtr(nullptr)
	, StoreInterfacePtr(nullptr)
	, TitleFileInterfacePtr(nullptr)
	, UserCloudInterfacePtr(nullptr)
	, EntitlementsInterfacePtr(nullptr)
	, PurchaseInterfacePtr(nullptr)
	, StoreInterfaceV2Ptr(nullptr)
{
	StopTicker();
}

IOnlineSessionPtr FOnlineSubsystemEOS::GetSessionInterface() const
{
	return SessionInterfacePtr;
}

IOnlineFriendsPtr FOnlineSubsystemEOS::GetFriendsInterface() const
{
	return UserManager;
}

IOnlineSharedCloudPtr FOnlineSubsystemEOS::GetSharedCloudInterface() const
{
	UE_LOG_ONLINE(Error, TEXT("Shared Cloud Interface Requested"));
	return nullptr;
}

IOnlineUserCloudPtr FOnlineSubsystemEOS::GetUserCloudInterface() const
{
	return UserCloudInterfacePtr;
}

IOnlineEntitlementsPtr FOnlineSubsystemEOS::GetEntitlementsInterface() const
{
	FEOSSettings Settings = UEOSSettings::GetSettings();
	if (Settings.bUseNewEcomFlow)
	{
		return EntitlementsInterfacePtr;
	}
	else
	{
		UE_CALL_ONCE([]()
		{
			UE_LOG_ONLINE(Warning, TEXT("Entitlements Interface Requested - only supported with bUseNewEcomFlow"));
		});
		return nullptr;
	}
}

IOnlineLeaderboardsPtr FOnlineSubsystemEOS::GetLeaderboardsInterface() const
{
	return LeaderboardsInterfacePtr;
}

IOnlineVoicePtr FOnlineSubsystemEOS::GetVoiceInterface() const
{
	return nullptr;
}

IOnlineExternalUIPtr FOnlineSubsystemEOS::GetExternalUIInterface() const
{
	return UserManager;
}

IOnlineIdentityPtr FOnlineSubsystemEOS::GetIdentityInterface() const
{
	return UserManager;
}

IOnlineTitleFilePtr FOnlineSubsystemEOS::GetTitleFileInterface() const
{
	return TitleFileInterfacePtr;
}

IOnlineStoreV2Ptr FOnlineSubsystemEOS::GetStoreV2Interface() const
{
	FEOSSettings Settings = UEOSSettings::GetSettings();
	if (Settings.bUseNewEcomFlow)
	{
		return StoreInterfaceV2Ptr;
	}
	else
	{
		return StoreInterfacePtr;
	}
}

IOnlinePurchasePtr FOnlineSubsystemEOS::GetPurchaseInterface() const
{
	FEOSSettings Settings = UEOSSettings::GetSettings();
	if (Settings.bUseNewEcomFlow)
	{
		return PurchaseInterfacePtr;
	}
	else
	{
		return StoreInterfacePtr;
	}
}

IOnlineAchievementsPtr FOnlineSubsystemEOS::GetAchievementsInterface() const
{
	return AchievementsInterfacePtr;
}

IOnlineUserPtr FOnlineSubsystemEOS::GetUserInterface() const
{
	return UserManager;
}

IOnlinePresencePtr FOnlineSubsystemEOS::GetPresenceInterface() const
{
	return UserManager;
}

IOnlineStatsPtr FOnlineSubsystemEOS::GetStatsInterface() const
{
	return StatsInterfacePtr;
}

IOnlinePlayerSanctionEOSPtr FOnlineSubsystemEOS::GetPlayerSanctionEOSInterface() const
{
	return PlayerSanctionEOSPtr;
}

IOnlinePlayerReportEOSPtr FOnlineSubsystemEOS::GetPlayerReportEOSInterface() const
{
	return PlayerReportInterfacePtr;
}

IVoiceChatUser* FOnlineSubsystemEOS::GetVoiceChatUserInterface(const FUniqueNetId& LocalUserId)
{
	IVoiceChatUser* Result = nullptr;

#if WITH_EOSVOICECHAT
	if (!VoiceChatInterface)
	{
		if (FEOSVoiceChatFactory* EOSVoiceChatFactory = FEOSVoiceChatFactory::Get())
		{
			VoiceChatInterface = EOSVoiceChatFactory->CreateInstanceWithPlatform(EOSPlatformHandle);

			if (VoiceChatInterface)
			{
				if (VoiceChatInterface->Initialize())
				{
					TOptional<bool> bConnectResult;
					VoiceChatInterface->Connect(FOnVoiceChatConnectCompleteDelegate::CreateLambda(
					[&bConnectResult](const FVoiceChatResult& VoiceChatResult)
					{
						bConnectResult = VoiceChatResult.IsSuccess();
					}));

					if (!bConnectResult.IsSet() || !bConnectResult.GetValue())
					{
						UE_LOG_ONLINE(Warning, TEXT("GetVoiceChatUserInterface: VoiceChat Connect failed"));
						VoiceChatInterface.Reset();
					}
				}
				else
				{
					UE_LOG_ONLINE(Warning, TEXT("GetVoiceChatUserInterface: VoiceChat Initialize failed"));
					VoiceChatInterface.Reset();
				}
			}
		}
	}

	if(VoiceChatInterface && UserManager->IsLocalUser(LocalUserId))
	{
		if (FOnlineSubsystemEOSVoiceChatUserWrapperRef* WrapperPtr = LocalVoiceChatUsers.Find(LocalUserId.AsShared()))
		{
			Result = &WrapperPtr->Get();
		}
		else
		{
			FEOSVoiceChatUser* VoiceChatUser = static_cast<FEOSVoiceChatUser*>(VoiceChatInterface->CreateUser());
			VoiceChatUser->Login(UserManager->GetPlatformUserIdFromUniqueNetId(LocalUserId), LexToString(FUniqueNetIdEOS::Cast(LocalUserId).GetProductUserId()), FString(), FOnVoiceChatLoginCompleteDelegate());

			const FOnlineSubsystemEOSVoiceChatUserWrapperRef& Wrapper = LocalVoiceChatUsers.Emplace(LocalUserId.AsShared(), MakeShared<FOnlineSubsystemEOSVoiceChatUserWrapper, ESPMode::ThreadSafe>(*VoiceChatUser));
			Result = &Wrapper.Get();
		}
	}
#endif // WITH_EOSVOICECHAT

	return Result;
}

void FOnlineSubsystemEOS::QueryUniqueNetId(int32 LocalUserNum, const EOS_ProductUserId& ProductUserId, const FOnQueryUniqueNetIdComplete & Callback)
{
	if (UserManager)
	{
		FUserManagerEOS::FResolveUniqueNetIdCallback ResolveUniqueNetIdCallback = [Callback](FUniqueNetIdEOSRef ResolvedUniqueNetId, const FOnlineError& Error)
		{
			Callback.ExecuteIfBound(ResolvedUniqueNetId, Error);
		};

		UserManager->ResolveUniqueNetId(LocalUserNum, ProductUserId, ResolveUniqueNetIdCallback);
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("Unable to query UniqueNetId for ProductUserId. UserManager not valid."));

		ExecuteNextTick([Callback]()
		{
			Callback.ExecuteIfBound(FUniqueNetIdEOS::EmptyId(), FOnlineError(EOnlineErrorResult::MissingInterface));
		});
	}
}

FEOSVoiceChatUser* FOnlineSubsystemEOS::GetEOSVoiceChatUserInterface(const FUniqueNetId& LocalUserId)
{
	FEOSVoiceChatUser* Result = nullptr;
#if WITH_EOSVOICECHAT
	if (IVoiceChatUser* Wrapper = GetVoiceChatUserInterface(LocalUserId))
	{
		Result = &static_cast<FOnlineSubsystemEOSVoiceChatUserWrapper*>(Wrapper)->VoiceChatUser;
	}
#endif // WITH_EOSVOICECHAT
	return Result;
}

void FOnlineSubsystemEOS::ReleaseVoiceChatUserInterface(const FUniqueNetId& LocalUserId)
{
#if WITH_EOSVOICECHAT
	if (VoiceChatInterface)
	{
		if (FOnlineSubsystemEOSVoiceChatUserWrapperRef* WrapperPtr = LocalVoiceChatUsers.Find(LocalUserId.AsShared()))
		{
			VoiceChatInterface->ReleaseUser(&(**WrapperPtr).VoiceChatUser);
			LocalVoiceChatUsers.Remove(LocalUserId.AsShared());
		}
	}
#endif // WITH_EOSVOICECHAT
}

#endif // WITH_EOS_SDK
