// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IOnlineSubsystemEOS.h"
#include "OnlineSubsystemNames.h"

#include COMPILED_PLATFORM_HEADER(EOSHelpers.h)

DECLARE_STATS_GROUP(TEXT("EOS"), STATGROUP_EOS, STATCAT_Advanced);

#if WITH_EOS_SDK

#include "eos_sdk.h"

class FSocketSubsystemEOS;
class IEOSSDKManager;
using IEOSPlatformHandlePtr = TSharedPtr<class IEOSPlatformHandle, ESPMode::ThreadSafe>;

class IVoiceChatUser;
class FEOSVoiceChatUser;
using IVoiceChatPtr = TSharedPtr<class IVoiceChat, ESPMode::ThreadSafe>;
using FOnlineSubsystemEOSVoiceChatUserWrapperRef = TSharedRef<class FOnlineSubsystemEOSVoiceChatUserWrapper, ESPMode::ThreadSafe>;

class FUserManagerEOS;
typedef TSharedPtr<class FUserManagerEOS, ESPMode::ThreadSafe> FUserManagerEOSPtr;

class FOnlineSessionEOS;
typedef TSharedPtr<class FOnlineSessionEOS, ESPMode::ThreadSafe> FOnlineSessionEOSPtr;

class FOnlineStatsEOS;
typedef TSharedPtr<class FOnlineStatsEOS, ESPMode::ThreadSafe> FOnlineStatsEOSPtr;

class FOnlineLeaderboardsEOS;
typedef TSharedPtr<class FOnlineLeaderboardsEOS, ESPMode::ThreadSafe> FOnlineLeaderboardsEOSPtr;

class FOnlineAchievementsEOS;
typedef TSharedPtr<class FOnlineAchievementsEOS, ESPMode::ThreadSafe> FOnlineAchievementsEOSPtr;

class FOnlineStoreEOS;
typedef TSharedPtr<class FOnlineStoreEOS, ESPMode::ThreadSafe> FOnlineStoreEOSPtr;

class FOnlineTitleFileEOS;
typedef TSharedPtr<class FOnlineTitleFileEOS, ESPMode::ThreadSafe> FOnlineTitleFileEOSPtr;

class FOnlineUserCloudEOS;
typedef TSharedPtr<class FOnlineUserCloudEOS, ESPMode::ThreadSafe> FOnlineUserCloudEOSPtr;

class FOnlinePlayerSanctionEOS;
typedef TSharedPtr<class FOnlinePlayerSanctionEOS, ESPMode::ThreadSafe> FOnlinePlayerSanctionEOSPtr;

class FOnlinePlayerReportEOS;
typedef TSharedPtr<class FOnlinePlayerReportEOS, ESPMode::ThreadSafe> FOnlinePlayerReportEOSPtr;

// FOnlineEntitlementsEOS, FOnlinePurchaseEOS and FOnlineStoreEOSV2 are only set if bUseNewEcomFlow is true
class FOnlineEntitlementsEOS;
typedef TSharedPtr<class FOnlineEntitlementsEOS, ESPMode::ThreadSafe> FOnlineEntitlementsEOSPtr;

class FOnlinePurchaseEOS;
typedef TSharedPtr<class FOnlinePurchaseEOS, ESPMode::ThreadSafe> FOnlinePurchaseEOSPtr;

class FOnlineStoreEOSV2;
typedef TSharedPtr<class FOnlineStoreEOSV2, ESPMode::ThreadSafe> FOnlineStoreEOSV2Ptr;

typedef TSharedPtr<FPlatformEOSHelpers, ESPMode::ThreadSafe> FPlatformEOSHelpersPtr;

/**
 *	OnlineSubsystemEOS - Implementation of the online subsystem for EOS services
 */
class FOnlineSubsystemEOS : 
	public IOnlineSubsystemEOS
{
public:
	virtual ~FOnlineSubsystemEOS() = default;

	/** Used to be called before RHIInit() */
	static ONLINESUBSYSTEMEOS_API void ModuleInit();
	static ONLINESUBSYSTEMEOS_API void ModuleShutdown();

	FPlatformEOSHelpersPtr GetEOSHelpers() { return EOSHelpersPtr; };

// IOnlineSubsystemEOS
	ONLINESUBSYSTEMEOS_API virtual IVoiceChatUser* GetVoiceChatUserInterface(const FUniqueNetId& LocalUserId) override;
	virtual IEOSPlatformHandlePtr GetEOSPlatformHandle() const override { return EOSPlatformHandle; };
	ONLINESUBSYSTEMEOS_API virtual void QueryUniqueNetId(int32 LocalUserNum, const EOS_ProductUserId& ProductUserId, const FOnQueryUniqueNetIdComplete & Callback) override;

// IOnlineSubsystem
	ONLINESUBSYSTEMEOS_API virtual IOnlineSessionPtr GetSessionInterface() const override;
	ONLINESUBSYSTEMEOS_API virtual IOnlineFriendsPtr GetFriendsInterface() const override;
	ONLINESUBSYSTEMEOS_API virtual IOnlineSharedCloudPtr GetSharedCloudInterface() const override;
	ONLINESUBSYSTEMEOS_API virtual IOnlineUserCloudPtr GetUserCloudInterface() const override;
	ONLINESUBSYSTEMEOS_API virtual IOnlineEntitlementsPtr GetEntitlementsInterface() const override;
	ONLINESUBSYSTEMEOS_API virtual IOnlineLeaderboardsPtr GetLeaderboardsInterface() const override;
	ONLINESUBSYSTEMEOS_API virtual IOnlineVoicePtr GetVoiceInterface() const override;
	ONLINESUBSYSTEMEOS_API virtual IOnlineExternalUIPtr GetExternalUIInterface() const override;	
	ONLINESUBSYSTEMEOS_API virtual IOnlineIdentityPtr GetIdentityInterface() const override;
	ONLINESUBSYSTEMEOS_API virtual IOnlineTitleFilePtr GetTitleFileInterface() const override;
	ONLINESUBSYSTEMEOS_API virtual IOnlineStoreV2Ptr GetStoreV2Interface() const override;
	ONLINESUBSYSTEMEOS_API virtual IOnlinePurchasePtr GetPurchaseInterface() const override;
	ONLINESUBSYSTEMEOS_API virtual IOnlineAchievementsPtr GetAchievementsInterface() const override;
	ONLINESUBSYSTEMEOS_API virtual IOnlineUserPtr GetUserInterface() const override;
	ONLINESUBSYSTEMEOS_API virtual IOnlinePresencePtr GetPresenceInterface() const override;
	ONLINESUBSYSTEMEOS_API virtual FText GetOnlineServiceName() const override;
	ONLINESUBSYSTEMEOS_API virtual IOnlineStatsPtr GetStatsInterface() const override;
	ONLINESUBSYSTEMEOS_API virtual IOnlinePlayerSanctionEOSPtr GetPlayerSanctionEOSInterface() const override;
	ONLINESUBSYSTEMEOS_API virtual IOnlinePlayerReportEOSPtr GetPlayerReportEOSInterface() const override;
	ONLINESUBSYSTEMEOS_API virtual bool Exec(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	ONLINESUBSYSTEMEOS_API virtual void ReloadConfigs(const TSet<FString>& ConfigSections) override;

	virtual IOnlineGroupsPtr GetGroupsInterface() const override { return nullptr; }
	virtual IOnlinePartyPtr GetPartyInterface() const override { return nullptr; }
	virtual IOnlineTimePtr GetTimeInterface() const override { return nullptr; }
	virtual IOnlineEventsPtr GetEventsInterface() const override { return nullptr; }
	virtual IOnlineSharingPtr GetSharingInterface() const override { return nullptr; }
	virtual IOnlineMessagePtr GetMessageInterface() const override { return nullptr; }
	virtual IOnlineChatPtr GetChatInterface() const override { return nullptr; }
	virtual IOnlineTurnBasedPtr GetTurnBasedInterface() const override { return nullptr; }
	virtual IOnlineTournamentPtr GetTournamentInterface() const override { return nullptr; }
//~IOnlineSubsystem

	ONLINESUBSYSTEMEOS_API virtual bool Init() override;
	ONLINESUBSYSTEMEOS_API virtual bool Shutdown() override;
	ONLINESUBSYSTEMEOS_API virtual FString GetAppId() const override;

// FTSTickerObjectBase
	ONLINESUBSYSTEMEOS_API virtual bool Tick(float DeltaTime) override;

	/** Only the factory makes instances */
	FOnlineSubsystemEOS() = delete;
	ONLINESUBSYSTEMEOS_API explicit FOnlineSubsystemEOS(FName InInstanceName);

	FString ProductId;

	IEOSSDKManager* EOSSDKManager;

	/** EOS handles */
	IEOSPlatformHandlePtr EOSPlatformHandle;
	EOS_HAuth AuthHandle;
	EOS_HUI UIHandle;
	EOS_HFriends FriendsHandle;
	EOS_HUserInfo UserInfoHandle;
	EOS_HPresence PresenceHandle;
	EOS_HConnect ConnectHandle;
	EOS_HSessions SessionsHandle;
	EOS_HStats StatsHandle;
	EOS_HLeaderboards LeaderboardsHandle;
	EOS_HMetrics MetricsHandle;
	EOS_HAchievements AchievementsHandle;
	EOS_HEcom EcomHandle;
	EOS_HTitleStorage TitleStorageHandle;
	EOS_HPlayerDataStorage PlayerDataStorageHandle;
	EOS_HSanctions PlayerSanctionHandle;
	EOS_HReports PlayerReportHandle;

	/** Manager that handles all user interfaces */
	FUserManagerEOSPtr UserManager;
	/** The session interface object */
	FOnlineSessionEOSPtr SessionInterfacePtr;
	/** Stats interface pointer */
	FOnlineStatsEOSPtr StatsInterfacePtr;
	/** Leaderboards interface pointer */
	FOnlineLeaderboardsEOSPtr LeaderboardsInterfacePtr;
	FOnlineAchievementsEOSPtr AchievementsInterfacePtr;
	/** EGS store interface pointer */
	FOnlineStoreEOSPtr StoreInterfacePtr;
	/** Title File interface pointer */
	FOnlineTitleFileEOSPtr TitleFileInterfacePtr;
	/** User Cloud interface pointer */
	FOnlineUserCloudEOSPtr UserCloudInterfacePtr;
	/** Player sanction interface pointer */
	FOnlinePlayerSanctionEOSPtr PlayerSanctionEOSPtr;
	/** Player Report interface pointer */
	FOnlinePlayerReportEOSPtr PlayerReportInterfacePtr;
	/** Entitlements interface pointer - only valid if bUseNewEcomFlow is true */
	FOnlineEntitlementsEOSPtr EntitlementsInterfacePtr;
	/** Purchase interface pointer - only valid if bUseNewEcomFlow is true */
	FOnlinePurchaseEOSPtr PurchaseInterfacePtr;
	/** Store interface pointer used to retrieve EGS offers - only valid if bUseNewEcomFlow is true */
	FOnlineStoreEOSV2Ptr StoreInterfaceV2Ptr;

	TSharedPtr<FSocketSubsystemEOS, ESPMode::ThreadSafe> SocketSubsystem;

	static ONLINESUBSYSTEMEOS_API FPlatformEOSHelpersPtr EOSHelpersPtr;

	ONLINESUBSYSTEMEOS_API FEOSVoiceChatUser* GetEOSVoiceChatUserInterface(const FUniqueNetId& LocalUserId);
	ONLINESUBSYSTEMEOS_API void ReleaseVoiceChatUserInterface(const FUniqueNetId& LocalUserId);

private:
	ONLINESUBSYSTEMEOS_API bool PlatformCreate();

	IVoiceChatPtr VoiceChatInterface;
	TUniqueNetIdMap<FOnlineSubsystemEOSVoiceChatUserWrapperRef> LocalVoiceChatUsers;
};

#else

class FOnlineSubsystemEOS :
	public FOnlineSubsystemImpl
{
public:
	explicit FOnlineSubsystemEOS(FName InInstanceName) :
		FOnlineSubsystemImpl(EOS_SUBSYSTEM, InInstanceName)
	{
	}

	virtual ~FOnlineSubsystemEOS() = default;

	virtual IOnlineSessionPtr GetSessionInterface() const override { return nullptr; }
	virtual IOnlineFriendsPtr GetFriendsInterface() const override { return nullptr; }
	virtual IOnlineGroupsPtr GetGroupsInterface() const override { return nullptr; }
	virtual IOnlinePartyPtr GetPartyInterface() const override { return nullptr; }
	virtual IOnlineSharedCloudPtr GetSharedCloudInterface() const override { return nullptr; }
	virtual IOnlineUserCloudPtr GetUserCloudInterface() const override { return nullptr; }
	virtual IOnlineEntitlementsPtr GetEntitlementsInterface() const override { return nullptr; }
	virtual IOnlineLeaderboardsPtr GetLeaderboardsInterface() const override { return nullptr; }
	virtual IOnlineVoicePtr GetVoiceInterface() const override { return nullptr; }
	virtual IOnlineExternalUIPtr GetExternalUIInterface() const override { return nullptr; }
	virtual IOnlineTimePtr GetTimeInterface() const override { return nullptr; }
	virtual IOnlineIdentityPtr GetIdentityInterface() const override { return nullptr; }
	virtual IOnlineTitleFilePtr GetTitleFileInterface() const override { return nullptr; }
	virtual IOnlineStoreV2Ptr GetStoreV2Interface() const override { return nullptr; }
	virtual IOnlinePurchasePtr GetPurchaseInterface() const override { return nullptr; }
	virtual IOnlineEntitlementsPtr GetEntitlementsInterface() const override { return nullptr; }
	virtual IOnlineEventsPtr GetEventsInterface() const override { return nullptr; }
	virtual IOnlineAchievementsPtr GetAchievementsInterface() const override { return nullptr; }
	virtual IOnlineSharingPtr GetSharingInterface() const override { return nullptr; }
	virtual IOnlineUserPtr GetUserInterface() const override { return nullptr; }
	virtual IOnlineMessagePtr GetMessageInterface() const override { return nullptr; }
	virtual IOnlinePresencePtr GetPresenceInterface() const override { return nullptr; }
	virtual IOnlineChatPtr GetChatInterface() const override { return nullptr; }
	virtual IOnlineStatsPtr GetStatsInterface() const override { return nullptr; }
	virtual IOnlineTurnBasedPtr GetTurnBasedInterface() const override { return nullptr; }
	virtual IOnlineTournamentPtr GetTournamentInterface() const override { return nullptr; }
	virtual FText GetOnlineServiceName() const override { return NSLOCTEXT("OnlineSubsystemEOS", "OnlineServiceName", "EOS"); }

	virtual bool Init() override { return false; }
	virtual bool Shutdown() override { return true; }
	virtual FString GetAppId() const override { return TEXT(""); }
};

#endif

typedef TSharedPtr<FOnlineSubsystemEOS, ESPMode::ThreadSafe> FOnlineSubsystemEOSPtr;

