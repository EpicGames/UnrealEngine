// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemEOSPrivate.h"
#include "OnlineSubsystemModule.h"
#include "OnlineSubsystemNames.h"
#include "OnlineSubsystemEOS.h"
#include "OnlineSubsystemEOSTypes.h"
#include "EOSSettings.h"

#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"

#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/LazySingleton.h"
#include "Modules/ModuleInterface.h"

#if WITH_EDITOR
	#include "ISettingsModule.h"
	#include "ISettingsSection.h"
#endif

#define LOCTEXT_NAMESPACE "EOS"

/**
 * Class responsible for creating instance(s) of the subsystem
 */
class FOnlineFactoryEOS :
	public IOnlineFactory
{
public:

	FOnlineFactoryEOS() {}
	virtual ~FOnlineFactoryEOS() {}

	virtual IOnlineSubsystemPtr CreateSubsystem(FName InstanceName)
	{
		FOnlineSubsystemEOSPtr OnlineSub = MakeShared<FOnlineSubsystemEOS, ESPMode::ThreadSafe>(InstanceName);
		if (!OnlineSub->Init())
		{
			UE_LOG_ONLINE(Warning, TEXT("EOS API failed to initialize!"));
			OnlineSub->Shutdown();
			OnlineSub = nullptr;
		}

		return OnlineSub;
	}
};

/**
 * Online subsystem module class  (EOS Implementation)
 * Code related to the loading of the EOS module
 */
class FOnlineSubsystemEOSModule : public IModuleInterface
{
public:
	FOnlineSubsystemEOSModule() = default;

	virtual ~FOnlineSubsystemEOSModule() = default;

#if WITH_EDITOR
	void OnPostEngineInit();
	void OnPreExit();
#endif

	// IModuleInterface

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

	virtual bool SupportsAutomaticShutdown() override
	{
		return false;
	}

private:
	/** Class responsible for creating instance(s) of the subsystem */
	TUniquePtr<FOnlineFactoryEOS> EOSFactory;
};

IMPLEMENT_MODULE(FOnlineSubsystemEOSModule, OnlineSubsystemEOS);

void FOnlineSubsystemEOSModule::StartupModule()
{
	if (FParse::Param(FCommandLine::Get(), TEXT("NoEOS")))
	{
		return;
	}

	EOSFactory = MakeUnique<FOnlineFactoryEOS>();

	// Create and register our singleton factory with the main online subsystem for easy access
	FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
	OSS.RegisterPlatformService(EOS_SUBSYSTEM, EOSFactory.Get());

#if WITH_EOS_SDK
	// Have to call this as early as possible in order to hook the rendering device
	FOnlineSubsystemEOS::ModuleInit();
	UEOSSettings::ModuleInit();
#endif

#if WITH_EDITOR
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FOnlineSubsystemEOSModule::OnPostEngineInit);
	FCoreDelegates::OnPreExit.AddRaw(this, &FOnlineSubsystemEOSModule::OnPreExit);
#endif
}

#if WITH_EDITOR
void FOnlineSubsystemEOSModule::OnPostEngineInit()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->RegisterSettings("Project", "Plugins", "Online Subsystem EOS",
			LOCTEXT("OSSEOSSettingsName", "Online Subsystem EOS"),
			LOCTEXT("OSSEOSSettingsDescription", "Configure Online Subsystem EOS"),
			GetMutableDefault<UEOSSettings>());
	}
}
#endif

#if WITH_EDITOR
void FOnlineSubsystemEOSModule::OnPreExit()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "Online Subsystem EOS");
	}
}
#endif

void FOnlineSubsystemEOSModule::ShutdownModule()
{
#if WITH_EDITOR
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	FCoreDelegates::OnPreExit.RemoveAll(this);
#endif

#if WITH_EOS_SDK
	FOnlineSubsystemEOS::ModuleShutdown();
	UEOSSettings::ModuleShutdown();
#endif

	FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
	OSS.UnregisterPlatformService(EOS_SUBSYSTEM);

	EOSFactory.Reset();

	TLazySingleton<FUniqueNetIdEOSRegistry>::TearDown();
}

#undef LOCTEXT_NAMESPACE
