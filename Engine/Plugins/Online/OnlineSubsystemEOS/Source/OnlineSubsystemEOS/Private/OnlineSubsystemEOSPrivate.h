// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OnlineSubsystem.h"
#include "Modules/ModuleManager.h"

#define INVALID_INDEX -1

/** URL Prefix when using EOS socket connection */
#define EOS_URL_PREFIX TEXT("EOS.")

/** pre-pended to all NULL logging */
#undef ONLINE_LOG_PREFIX
#define ONLINE_LOG_PREFIX TEXT("EOS: ")


