// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemEOSTypes.h"

#include "Algo/AnyOf.h"
#include "Misc/LazySingleton.h"
#include "Misc/ScopeRWLock.h"
#include "UserManagerEOS.h"

#define EOS_DedicatedServer_ID 0xD5D5D5D5D5D5D5D5
const uint8 EOS_DedicatedServer_ID_RawBytes[EOS_ID_BYTE_SIZE] = "D5D5D5D5D5D5D5D5D5D5D5D5D5D5D5D";

const FUniqueNetIdEOS& FUniqueNetIdEOS::Cast(const FUniqueNetId& NetId)
{
	check(GetTypeStatic() == NetId.GetType());
	return *static_cast<const FUniqueNetIdEOS*>(&NetId);
}

const FUniqueNetIdEOSRef& FUniqueNetIdEOS::EmptyId()
{
	static const FUniqueNetIdEOSRef EmptyId(Create());
	return EmptyId;
}

const FUniqueNetIdEOSRef& FUniqueNetIdEOS::DedicatedServerId()
{
	static const FUniqueNetIdEOSRef DedicatedServerId(Create(EOS_DedicatedServer_ID_RawBytes, EOS_ID_BYTE_SIZE));
	return DedicatedServerId;
}

FName FUniqueNetIdEOS::GetTypeStatic()
{
	static FName NAME_Eos(TEXT("EOS"));
	return NAME_Eos;
}

FName FUniqueNetIdEOS::GetType() const
{
	return GetTypeStatic();
}

const uint8* FUniqueNetIdEOS::GetBytes() const
{
	return RawBytes;
}

int32 FUniqueNetIdEOS::GetSize() const
{
	return EOS_ID_BYTE_SIZE;
}

bool FUniqueNetIdEOS::IsDSValue() const 
{
	return ProductUserId == EOS_ProductUserId(EOS_DedicatedServer_ID); 
}

bool FUniqueNetIdEOS::IsValid() const
{
	return IsDSValue() || EOS_EpicAccountId_IsValid(EpicAccountId) || EOS_ProductUserId_IsValid(ProductUserId);
}

const EOS_EpicAccountId FUniqueNetIdEOS::GetEpicAccountId() const
{
	return EpicAccountId;
}

const  EOS_ProductUserId FUniqueNetIdEOS::GetProductUserId() const
{
	return IsDSValue() ? nullptr: ProductUserId;
}

uint32 FUniqueNetIdEOS::GetTypeHash() const
{
	return ::GetTypeHash(static_cast<const void*>(this));
}

FString FUniqueNetIdEOS::ToString() const
{
	return LexToString(EpicAccountId) + EOS_ID_SEPARATOR + LexToString(ProductUserId);
}

FString FUniqueNetIdEOS::ToDebugString() const
{
	if (IsValid())
	{
		const FString EpicAccountIdStr = LexToString(EpicAccountId);
		const FString ProductUserIdStr = LexToString(ProductUserId);
		return OSS_UNIQUEID_REDACT(*this, EpicAccountIdStr) + EOS_ID_SEPARATOR + OSS_UNIQUEID_REDACT(*this, ProductUserIdStr);
	}
	else
	{
		return TEXT("INVALID");
	}
}

FUniqueNetIdEOS::FUniqueNetIdEOS(const uint8* Bytes, int32 Size)
{
	check(Size == EOS_ID_BYTE_SIZE);
	FMemory::Memcpy(RawBytes, Bytes, EOS_ID_BYTE_SIZE);

	if (FMemory::Memcmp(RawBytes, EOS_DedicatedServer_ID_RawBytes, EOS_ID_BYTE_SIZE) == 0)
	{
		EpicAccountId = nullptr;
		ProductUserId = EOS_ProductUserId(EOS_DedicatedServer_ID);
	}
	else
	{
		const bool bIsEasNonZero = Algo::AnyOf(TArrayView<const uint8>(Bytes, ID_HALF_BYTE_SIZE));
		if (bIsEasNonZero)
		{
			const FString EpicAccountIdStr = BytesToHexLower(Bytes, ID_HALF_BYTE_SIZE);
			EpicAccountId = EOS_EpicAccountId_FromString(TCHAR_TO_UTF8(*EpicAccountIdStr));
		}

		const bool bIsPuidNonZero = Algo::AnyOf(TArrayView<const uint8>(Bytes + ID_HALF_BYTE_SIZE, ID_HALF_BYTE_SIZE));
		if (bIsPuidNonZero)
		{
			const FString ProductUserIdStr = BytesToHexLower(Bytes + ID_HALF_BYTE_SIZE, ID_HALF_BYTE_SIZE);
			ProductUserId = EOS_ProductUserId_FromString(TCHAR_TO_UTF8(*ProductUserIdStr));
		}
	}
}

FUniqueNetIdEOS::FUniqueNetIdEOS(EOS_EpicAccountId InEpicAccountId, EOS_ProductUserId InProductUserId)
	: EpicAccountId(InEpicAccountId)
	, ProductUserId(InProductUserId)
{
	HexToBytes(LexToString(EpicAccountId), RawBytes);
	HexToBytes(LexToString(ProductUserId), RawBytes + ID_HALF_BYTE_SIZE);
}

FUniqueNetIdEOSRegistry& FUniqueNetIdEOSRegistry::Get()
{
	return TLazySingleton<FUniqueNetIdEOSRegistry>::Get();
}

FUniqueNetIdEOSRef FUniqueNetIdEOSRegistry::FindOrAddImpl(const FString& NetIdStr)
{
	FString EpicAccountIdStr;
	FString ProductUserIdStr;
	if (!NetIdStr.Split(EOS_ID_SEPARATOR, &EpicAccountIdStr, &ProductUserIdStr)
		|| (!EpicAccountIdStr.IsEmpty() && EpicAccountIdStr.Len() != EOS_EPICACCOUNTID_MAX_LENGTH)
		|| (!ProductUserIdStr.IsEmpty() && ProductUserIdStr.Len() != EOS_PRODUCTUSERID_MAX_LENGTH))
	{
		return FUniqueNetIdEOS::EmptyId();
	}

	EOS_EpicAccountId EpicAccountId = nullptr;
	if (!EpicAccountIdStr.IsEmpty())
	{
		EpicAccountId = EOS_EpicAccountId_FromString(TCHAR_TO_UTF8(*EpicAccountIdStr));
	}
	EOS_ProductUserId ProductUserId = nullptr;
	if (!ProductUserIdStr.IsEmpty())
	{
		ProductUserId = EOS_ProductUserId_FromString(TCHAR_TO_UTF8(*ProductUserIdStr));
	}

	return FindOrAddImpl(EpicAccountId, ProductUserId);
}

FUniqueNetIdEOSRef FUniqueNetIdEOSRegistry::FindOrAddImpl(const uint8* Bytes, int32 Size)
{
	if (Size == EOS_ID_BYTE_SIZE)
	{
		// The net id type already knows how to deserialize, so create a temp one to get the EAS/PUID
		FUniqueNetIdEOS Temp(Bytes, Size);
		if (Temp.IsDSValue())
		{
			return FUniqueNetIdEOS::DedicatedServerId(); 
		}
		return FindOrAddImpl(Temp.GetEpicAccountId(), Temp.GetProductUserId());
	}
	return FUniqueNetIdEOS::EmptyId();
}

FUniqueNetIdEOSRef FUniqueNetIdEOSRegistry::FindOrAddImpl(const EOS_EpicAccountId InEpicAccountId, const EOS_ProductUserId InProductUserId)
{
	FUniqueNetIdEOSPtr Result;

	const bool bInEpicAccountIdValid = EOS_EpicAccountId_IsValid(InEpicAccountId) == EOS_TRUE;
	const bool bInProductUserIdValid = EOS_ProductUserId_IsValid(InProductUserId) == EOS_TRUE;
	if (bInEpicAccountIdValid || bInProductUserIdValid)
	{
		bool bUpdateEpicAccountId = false;
		bool bUpdateProductUserId = false;

		auto FindExisting = [this, InEpicAccountId, InProductUserId, bInEpicAccountIdValid, bInProductUserIdValid, &bUpdateEpicAccountId, &bUpdateProductUserId]()
		{
			FUniqueNetIdEOSPtr Result;

			if (const FUniqueNetIdEOSRef* FoundEas = bInEpicAccountIdValid ? EasToNetId.Find(InEpicAccountId) : nullptr)
			{
				Result = *FoundEas;
			}
			else if (const FUniqueNetIdEOSRef* FoundProd = bInProductUserIdValid ? PuidToNetId.Find(InProductUserId) : nullptr)
			{
				Result = *FoundProd;
			}

			if (Result.IsValid())
			{
				const EOS_EpicAccountId FoundEpicAccountId = Result->GetEpicAccountId();
				const EOS_ProductUserId FoundProductUserId = Result->GetProductUserId();
				const bool bFoundEpicAccountIdValid = EOS_EpicAccountId_IsValid(FoundEpicAccountId) == EOS_TRUE;
				const bool bFoundProductUserIdValid = EOS_ProductUserId_IsValid(FoundProductUserId) == EOS_TRUE;

				// Check that the found EAS/EOS ids are either unset, or match the input. If a valid input is passed for a currently unset field, this is an update
				check(!bFoundEpicAccountIdValid || !bInEpicAccountIdValid || InEpicAccountId == FoundEpicAccountId);
				check(!bFoundProductUserIdValid || !bInProductUserIdValid || InProductUserId == FoundProductUserId);
				bUpdateEpicAccountId = !bFoundEpicAccountIdValid && bInEpicAccountIdValid;
				bUpdateProductUserId = !bFoundProductUserIdValid && bInProductUserIdValid;
			}

			return Result;
		};

		{
			// First take read lock and look for existing elements
			const FReadScopeLock ReadLock(Lock);
			Result = FindExisting();
		}

		if (!Result.IsValid())
		{
			// Double-checked locking. If we didn't find an element, we take the write lock, and look again, in case another thread raced with us and added one.
			const FWriteScopeLock WriteLock(Lock);
			Result = FindExisting();

			if (!Result.IsValid())
			{
				// if we didn't find one we can create a new one
				Result = FUniqueNetIdEOS::Create(InEpicAccountId, InProductUserId);
				if (bInEpicAccountIdValid)
				{
					EasToNetId.Emplace(InEpicAccountId, Result.ToSharedRef());
				}
				if (bInProductUserIdValid)
				{
					PuidToNetId.Emplace(InProductUserId, Result.ToSharedRef());
				}
			}
		}

		check(Result.IsValid());
		if (bUpdateEpicAccountId || bUpdateProductUserId)
		{
			// Finally, update any previously unset fields for which we now have a valid value.
			const FWriteScopeLock WriteLock(Lock);
			if (bUpdateEpicAccountId)
			{
				EasToNetId.Emplace(InEpicAccountId, Result.ToSharedRef());
				*ConstCastSharedPtr<FUniqueNetIdEOS>(Result) = FUniqueNetIdEOS(InEpicAccountId, Result->GetProductUserId());
			}
			if (bUpdateProductUserId)
			{
				PuidToNetId.Emplace(InProductUserId, Result.ToSharedRef());
				*ConstCastSharedPtr<FUniqueNetIdEOS>(Result) = FUniqueNetIdEOS(Result->GetEpicAccountId(), InProductUserId);
			}
		}
	}

	return Result ? Result.ToSharedRef() : FUniqueNetIdEOS::EmptyId();
}

FUniqueNetIdEOSPtr FUniqueNetIdEOSRegistry::FindImpl(EOS_EpicAccountId EpicAccountId)
{
	FUniqueNetIdEOSRef* Result = EasToNetId.Find(EpicAccountId);

	return Result ? FUniqueNetIdEOSPtr(*Result) : nullptr;
}

FUniqueNetIdEOSPtr FUniqueNetIdEOSRegistry::FindImpl(EOS_ProductUserId ProductUserId)
{
	FUniqueNetIdEOSRef* Result = PuidToNetId.Find(ProductUserId);

	return Result ? FUniqueNetIdEOSPtr(*Result) : nullptr;
}

FUniqueNetIdEOSRef FUniqueNetIdEOSRegistry::FindCheckedImpl(EOS_EpicAccountId EpicAccountId)
{
	FUniqueNetIdEOSRef* Result = EasToNetId.Find(EpicAccountId);
	check(Result != nullptr);

	if (Result != nullptr)
	{
		return *Result;
	}
	else
	{
		return FUniqueNetIdEOS::EmptyId();
	}
}

FUniqueNetIdEOSRef FUniqueNetIdEOSRegistry::FindCheckedImpl(EOS_ProductUserId ProductUserId)
{
	FUniqueNetIdEOSRef* Result = PuidToNetId.Find(ProductUserId);
	check(Result != nullptr);

	if (Result != nullptr)
	{
		return *Result;
	}
	else
	{
		return FUniqueNetIdEOS::EmptyId();
	}
}

namespace OnlineSubsystemEOSTypesPrivate
{
FString GetBestDisplayName(const FOnlineSubsystemEOS& EOSSubsystem, const EOS_EpicAccountId TargetUserId, const FStringView Platform)
{
	return EOSSubsystem.UserManager->GetBestDisplayName(TargetUserId, Platform);
}
} // namespace OnlineSubsystemEOSTypesPrivate