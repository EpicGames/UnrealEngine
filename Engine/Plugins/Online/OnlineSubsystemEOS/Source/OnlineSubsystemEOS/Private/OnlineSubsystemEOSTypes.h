// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemEOSTypesPublic.h"
#include "Interfaces/OnlineIdentityInterface.h"
#include "Interfaces/OnlinePresenceInterface.h"
#include "Interfaces/OnlineUserInterface.h"
#include "EOSShared.h"
#include "EOSSharedTypes.h"
#include "IPAddress.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemEOS.h"
#include "OnlineSubsystemTypes.h"
#include "OnlineSubsystemEOSPackage.h" // IWYU pragma: keep

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_sessions_types.h"

#define EOS_OSS_BUCKET_ID_STRING_LENGTH 60
#define EOS_OSS_STRING_BUFFER_LENGTH 256 + 1 // 256 plus null terminator

class FOnlineSubsystemEOS;

#define EOS_ID_SEPARATOR TEXT("|")
#define ID_HALF_BYTE_SIZE 16
#define EOS_ID_BYTE_SIZE (ID_HALF_BYTE_SIZE * 2)

typedef TSharedPtr<const class FUniqueNetIdEOS> FUniqueNetIdEOSPtr;
typedef TSharedRef<const class FUniqueNetIdEOS> FUniqueNetIdEOSRef;

/**
 * Unique net id wrapper for a EOS account ids.
 */
class FUniqueNetIdEOS : public IUniqueNetIdEOS
{
public:
	static const FUniqueNetIdEOS& Cast(const FUniqueNetId& NetId);

	/** global static instance of invalid (zero) id */
	static const FUniqueNetIdEOSRef& EmptyId();
	static const FUniqueNetIdEOSRef& DedicatedServerId();

	static FName GetTypeStatic();

	virtual FName GetType() const override;
	virtual const uint8* GetBytes() const override;
	virtual int32 GetSize() const override;

	virtual bool IsValid() const override;
	virtual uint32 GetTypeHash() const override;
	virtual FString ToString() const override;
	virtual FString ToDebugString() const override;
	virtual const EOS_EpicAccountId GetEpicAccountId() const override;
	virtual const EOS_ProductUserId GetProductUserId() const override;

private:
	EOS_EpicAccountId EpicAccountId = nullptr;
	EOS_ProductUserId ProductUserId = nullptr;
	uint8 RawBytes[EOS_ID_BYTE_SIZE] = { 0 };

	friend class FUniqueNetIdEOSRegistry;

	template<typename... TArgs>
	static FUniqueNetIdEOSRef Create(TArgs&&... Args)
	{
		return MakeShareable(new FUniqueNetIdEOS(Forward<TArgs>(Args)...));
	}

	FUniqueNetIdEOS() = default;
	
	bool IsDSValue() const;
	explicit FUniqueNetIdEOS(const uint8* Bytes, int32 Size);
	explicit FUniqueNetIdEOS(EOS_EpicAccountId InEpicAccountId, EOS_ProductUserId InProductUserId);
};

class FUniqueNetIdEOSRegistry
{
public:
	static FUniqueNetIdEOSRef FindOrAdd(const FString& NetIdStr) { return Get().FindOrAddImpl(NetIdStr); }
	static FUniqueNetIdEOSRef FindOrAdd(const uint8* Bytes, int32 Size) { return Get().FindOrAddImpl(Bytes, Size); }
	static FUniqueNetIdEOSRef FindOrAdd(EOS_EpicAccountId EpicAccountId, EOS_ProductUserId ProductUserId) { return Get().FindOrAddImpl(EpicAccountId, ProductUserId); }

	static FUniqueNetIdEOSPtr Find(EOS_EpicAccountId EpicAccountId) { return Get().FindImpl(EpicAccountId); }
	static FUniqueNetIdEOSPtr Find(EOS_ProductUserId ProductUserId) { return Get().FindImpl(ProductUserId); }

	static FUniqueNetIdEOSRef FindChecked(EOS_EpicAccountId EpicAccountId) { return Get().FindCheckedImpl(EpicAccountId); }
	static FUniqueNetIdEOSRef FindChecked(EOS_ProductUserId ProductUserId) { return Get().FindCheckedImpl(ProductUserId); }

private:
	FRWLock Lock;
	TMap<EOS_EpicAccountId, FUniqueNetIdEOSRef> EasToNetId;
	TMap<EOS_ProductUserId, FUniqueNetIdEOSRef> PuidToNetId;

	static FUniqueNetIdEOSRegistry& Get();

	FUniqueNetIdEOSRef FindOrAddImpl(const FString& NetIdStr);
	FUniqueNetIdEOSRef FindOrAddImpl(const uint8* Bytes, int32 Size);
	FUniqueNetIdEOSRef FindOrAddImpl(EOS_EpicAccountId EpicAccountId, EOS_ProductUserId ProductUserId);

	FUniqueNetIdEOSPtr FindImpl(EOS_EpicAccountId EpicAccountId);
	FUniqueNetIdEOSPtr FindImpl(EOS_ProductUserId ProductUserId);

	FUniqueNetIdEOSRef FindCheckedImpl(EOS_EpicAccountId EpicAccountId);
	FUniqueNetIdEOSRef FindCheckedImpl(EOS_ProductUserId ProductUserId);
};

#ifndef AUTH_ATTR_REFRESH_TOKEN
	#define AUTH_ATTR_REFRESH_TOKEN TEXT("refresh_token")
#endif
#ifndef AUTH_ATTR_ID_TOKEN
	#define AUTH_ATTR_ID_TOKEN TEXT("id_token")
#endif
#ifndef AUTH_ATTR_AUTHORIZATION_CODE
	#define AUTH_ATTR_AUTHORIZATION_CODE TEXT("authorization_code")
#endif

#define USER_ATTR_DISPLAY_NAME TEXT("display_name")
#define USER_ATTR_COUNTRY TEXT("country")
#define USER_ATTR_LANG TEXT("language")

#define SESSION_ATTR_SERVERPORT TEXT("port")

#if WITH_EOS_SDK

/** Used to update all types of FOnlineUser classes, irrespective of leaf most class type */
class IAttributeAccessInterface
{
public:
	virtual TMap<FString, FString> GetInternalAttributes() const
	{
		return TMap<FString, FString>();
	}

	virtual void UpdateInternalAttributes(const TMap<FString, FString>& InternalAttributes)
	{
	}

	virtual void SetInternalAttribute(const FString& AttrName, const FString& AttrValue)
	{
	}

	virtual FUniqueNetIdEOSPtr GetUniqueNetIdEOS() const
	{
		return FUniqueNetIdEOSPtr();
	}
};

typedef TSharedPtr<IAttributeAccessInterface> IAttributeAccessInterfacePtr;
typedef TSharedRef<IAttributeAccessInterface> IAttributeAccessInterfaceRef;

namespace OnlineSubsystemEOSTypesPrivate
{
FString GetBestDisplayName(const FOnlineSubsystemEOS& EOSSubsystem, const EOS_EpicAccountId TargetUserId, const FStringView Platform);
} // namespace OnlineSubsystemEOSTypesPrivate

/**
 * Implementation of FOnlineUser that can be shared across multiple class hiearchies
 */
template<class BaseClass, class AttributeAccessClass>
class TOnlineUserEOS
	: public BaseClass
	, public AttributeAccessClass
{
public:
	friend class FUserManagerEOS;

	TOnlineUserEOS(const FUniqueNetIdEOSRef& InNetIdRef, const FOnlineSubsystemEOS& InSubsystem)
		: UserIdRef(InNetIdRef)
		, EOSSubsystem(InSubsystem)
	{
	}

	TOnlineUserEOS(const FUniqueNetIdEOSRef& InNetIdRef, const TMap<FString, FString>& InUserAttributes, const FOnlineSubsystemEOS& InSubsystem)
		: UserIdRef(InNetIdRef)
		, UserAttributes(InUserAttributes)
		, EOSSubsystem(InSubsystem)
	{
	}

	virtual ~TOnlineUserEOS() = default;

// FOnlineUser
	virtual FUniqueNetIdRef GetUserId() const override
	{
		return UserIdRef;
	}

	virtual FString GetRealName() const override
	{
		return FString();
	}

	virtual FString GetDisplayName(const FString& Platform = FString()) const override
	{
		return OnlineSubsystemEOSTypesPrivate::GetBestDisplayName(EOSSubsystem, UserIdRef->GetEpicAccountId(), Platform);
	}

	virtual bool GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const override
	{
		const FString* FoundAttr = UserAttributes.Find(AttrName);
		if (FoundAttr != nullptr)
		{
			OutAttrValue = *FoundAttr;
			return true;
		}
		return false;
	}
//~FOnlineUser
	virtual TMap<FString, FString> GetInternalAttributes() const override
	{
		return UserAttributes;
	}

	virtual void UpdateInternalAttributes(const TMap<FString, FString>& InternalAttributes) override
	{
		UserAttributes.Append(InternalAttributes);
	}

	virtual void SetInternalAttribute(const FString& AttrName, const FString& AttrValue)
	{
		UserAttributes.Add(AttrName, AttrValue);
	}

	virtual FUniqueNetIdEOSPtr GetUniqueNetIdEOS() const
	{
		return UserIdRef;
	}

protected:
	/** User Id represented as a FUniqueNetId */
	FUniqueNetIdEOSRef UserIdRef;
	/** Additional key/value pair data related to user attribution */
	TMap<FString, FString> UserAttributes;

	const FOnlineSubsystemEOS& EOSSubsystem;
};

/**
 * Implementation of FUserOnlineAccount methods that adds in the online user template to complete the interface
 */
template<class BaseClass>
class TUserOnlineAccountEOS :
	public TOnlineUserEOS<BaseClass, IAttributeAccessInterface>
{
	using Super = TOnlineUserEOS<BaseClass, IAttributeAccessInterface>;

public:
	TUserOnlineAccountEOS(const FUniqueNetIdEOSRef& InNetIdRef, const FOnlineSubsystemEOS& InSubsystem)
		: Super(InNetIdRef, InSubsystem)
	{
	}

// FUserOnlineAccount
	virtual FString GetAccessToken() const override
	{
		FString Token;

		if (const IOnlineIdentityPtr Identity = Super::EOSSubsystem.GetIdentityInterface())
		{
			const int32 LocalUserNum = Identity->GetLocalUserNumFromPlatformUserId(Identity->GetPlatformUserIdFromUniqueNetId(*this->UserIdRef));
			Token = Identity->GetAuthToken(LocalUserNum);
		}

		return Token;
	}

	virtual bool GetAuthAttribute(const FString& AttrName, FString& OutAttrValue) const override
	{
		const FString* FoundAttr = AdditionalAuthData.Find(AttrName);
		if (FoundAttr != nullptr)
		{
			OutAttrValue = *FoundAttr;
			return true;
		}
		return false;
	}

	virtual bool SetUserAttribute(const FString& AttrName, const FString& AttrValue) override
	{
		const FString* FoundAttr = this->UserAttributes.Find(AttrName);
		if (FoundAttr == nullptr || *FoundAttr != AttrValue)
		{
			this->UserAttributes.Add(AttrName, AttrValue);
			return true;
		}
		return false;
	}
//~FUserOnlineAccount

	void SetAuthAttribute(const FString& AttrName, const FString& AttrValue)
	{
		AdditionalAuthData.Add(AttrName, AttrValue);
	}

protected:
	/** Additional key/value pair data related to auth */
	TMap<FString, FString> AdditionalAuthData;
};

typedef TSharedRef<FOnlineUserPresence> FOnlineUserPresenceRef;

/**
 * Implementation of FOnlineFriend methods that adds in the online user template to complete the interface
 */
template<class BaseClass>
class TOnlineFriendEOS :
	public TOnlineUserEOS<BaseClass, IAttributeAccessInterface>
{
public:
	TOnlineFriendEOS(const FUniqueNetIdEOSRef& InNetIdRef, const FOnlineSubsystemEOS& InSubsystem)
		: TOnlineUserEOS<BaseClass, IAttributeAccessInterface>(InNetIdRef, InSubsystem)
	{
	}

	TOnlineFriendEOS(const FUniqueNetIdEOSRef& InNetIdRef, const TMap<FString, FString>& InUserAttributes, const FOnlineSubsystemEOS& InSubsystem)
		: TOnlineUserEOS<BaseClass, IAttributeAccessInterface>(InNetIdRef, InUserAttributes, InSubsystem)
	{
	}

// FOnlineFriend
	/**
	 * @return the current invite status of a friend wrt to user that queried
	 */
	virtual EInviteStatus::Type GetInviteStatus() const override
	{
		return InviteStatus;
	}

	/**
	 * @return presence info for an online friend
	 */
	virtual const FOnlineUserPresence& GetPresence() const override
	{
		return Presence;
	}
//~FOnlineFriend

	void SetInviteStatus(EInviteStatus::Type InStatus)
	{
		InviteStatus = InStatus;
	}

	void SetPresence(FOnlineUserPresenceRef InPresence)
	{
		// Copy the data over since the friend shares it as a const&
		Presence = *InPresence;
	}

protected:
	FOnlineUserPresence Presence;
	EInviteStatus::Type InviteStatus;
};

namespace OSSInternalCallback
{
	/** Create a callback for a non-SDK function that is tied to the lifetime of an arbitrary shared pointer. */
	template <typename DelegateType, typename OwnerType, typename... CallbackArgs>
	[[nodiscard]] DelegateType Create(const TSharedPtr<OwnerType, ESPMode::ThreadSafe>& InOwner,
		const TFunction<void(CallbackArgs...)>& InUserCallback)
	{
		const DelegateType& CheckOwnerThenExecute = DelegateType::CreateLambda(
			[WeakOwner = TWeakPtr<OwnerType, ESPMode::ThreadSafe>(InOwner), InUserCallback](CallbackArgs... Payload) {
				check(IsInGameThread());
				TSharedPtr<OwnerType, ESPMode::ThreadSafe> Owner = WeakOwner.Pin();
				if (Owner.IsValid())
				{
					InUserCallback(Payload...);
				}
		});

		return CheckOwnerThenExecute;
	}
}

/**
 * Class to handle nested callbacks (callbacks that are tied to an external callback's lifetime,
 * e.g. file chunkers) generically using a lambda to process callback results
 */
template<typename CallbackFuncType, typename CallbackType, typename OwningType,
	typename Nested1CallbackFuncType, typename Nested1CallbackType, typename Nested1ReturnType>
class TEOSCallbackWithNested1 :
	public TEOSCallback<CallbackFuncType, CallbackType, OwningType>
{
public:
	TEOSCallbackWithNested1(TWeakPtr<OwningType> InOwner)
		: TEOSCallback<CallbackFuncType, CallbackType, OwningType>(InOwner)
	{
	}
	virtual ~TEOSCallbackWithNested1() = default;


	Nested1CallbackFuncType GetNested1CallbackPtr()
	{
		return &Nested1CallbackImpl;
	}

	void SetNested1CallbackLambda(TFunction<Nested1ReturnType(const Nested1CallbackType*)> InLambda)
	{
		Nested1CallbackLambda = InLambda;
	}

private:
	TFunction<Nested1ReturnType(const Nested1CallbackType*)> Nested1CallbackLambda;

	static Nested1ReturnType EOS_CALL Nested1CallbackImpl(const Nested1CallbackType* Data)
	{
		check(IsInGameThread());
		TEOSCallbackWithNested1* CallbackThis = (TEOSCallbackWithNested1*)Data->ClientData;
		check(CallbackThis);

		if (!CallbackThis->Owner.IsValid())
		{
			return Nested1ReturnType();
		}

		check(CallbackThis->CallbackLambda);
		return CallbackThis->Nested1CallbackLambda(Data);
	}
};

/**
 * Class to handle 2 nested callbacks (callbacks that are tied to an external callback's lifetime,
 * e.g. file chunkers) generically using a lambda to process callback results
 */
template<typename CallbackFuncType, typename CallbackType, typename OwningType,
	typename Nested1CallbackFuncType, typename Nested1CallbackType, typename Nested1ReturnType,
	typename Nested2CallbackFuncType, typename Nested2CallbackType>
class TEOSCallbackWithNested2 :
	public TEOSCallbackWithNested1<CallbackFuncType, CallbackType, OwningType, Nested1CallbackFuncType, Nested1CallbackType, Nested1ReturnType>
{
public:
	TEOSCallbackWithNested2(TWeakPtr<OwningType> InOwner)
		: TEOSCallbackWithNested1<CallbackFuncType, CallbackType, OwningType, Nested1CallbackFuncType, Nested1CallbackType, Nested1ReturnType>(InOwner)
	{
	}
	virtual ~TEOSCallbackWithNested2() = default;


	Nested2CallbackFuncType GetNested2CallbackPtr()
	{
		return &Nested2CallbackImpl;
	}

	void SetNested2CallbackLambda(TFunction<void(const Nested2CallbackType*)> InLambda)
	{
		Nested2CallbackLambda = InLambda;
	}

private:
	TFunction<void(const Nested2CallbackType*)> Nested2CallbackLambda;

	static void EOS_CALL Nested2CallbackImpl(const Nested2CallbackType* Data)
	{
		check(IsInGameThread());
		TEOSCallbackWithNested2* CallbackThis = (TEOSCallbackWithNested2*)Data->ClientData;
		check(CallbackThis);

		if (CallbackThis->Owner.IsValid())
		{
			check(CallbackThis->CallbackLambda);
			CallbackThis->Nested2CallbackLambda(Data);
		}
	}
};

/**
 * Class to handle nested callbacks (callbacks that are tied to an external callback's lifetime,
 * e.g. file chunkers) generically using a lambda to process callback results
 */
template<typename CallbackFuncType, typename CallbackType, typename OwningType,
	typename Nested1CallbackFuncType, typename Nested1CallbackType, typename Nested1ReturnType>
class TEOSCallbackWithNested1Param3 :
	public TEOSCallback<CallbackFuncType, CallbackType, OwningType>
{
public:
	TEOSCallbackWithNested1Param3(TWeakPtr<OwningType> InOwner)
		: TEOSCallback<CallbackFuncType, CallbackType, OwningType>(InOwner)
	{
	}
	virtual ~TEOSCallbackWithNested1Param3() = default;


	Nested1CallbackFuncType GetNested1CallbackPtr()
	{
		return (Nested1CallbackFuncType)&Nested1CallbackImpl;
	}

	void SetNested1CallbackLambda(TFunction<Nested1ReturnType(const Nested1CallbackType*, void*, uint32_t*)> InLambda)
	{
		Nested1CallbackLambda = InLambda;
	}

private:
	TFunction<Nested1ReturnType(const Nested1CallbackType*, void*, uint32_t*)> Nested1CallbackLambda;

	static Nested1ReturnType EOS_CALL Nested1CallbackImpl(const Nested1CallbackType* Data, void* OutDataBuffer, uint32_t* OutDataWritten)
	{
		check(IsInGameThread());
		TEOSCallbackWithNested1Param3* CallbackThis = (TEOSCallbackWithNested1Param3*)Data->ClientData;
		check(CallbackThis);

		if (!CallbackThis->Owner.IsValid())
		{
			return Nested1ReturnType();
		}

		check(CallbackThis->CallbackLambda);
		return CallbackThis->Nested1CallbackLambda(Data, OutDataBuffer, OutDataWritten);
	}
};

/**
 * Class to handle 2 nested callbacks (callbacks that are tied to an external callback's lifetime,
 * e.g. file chunkers) generically using a lambda to process callback results
 */
template<typename CallbackFuncType, typename CallbackType, typename OwningType,
	typename Nested1CallbackFuncType, typename Nested1CallbackType, typename Nested1ReturnType,
	typename Nested2CallbackFuncType, typename Nested2CallbackType>
class TEOSCallbackWithNested2ForNested1Param3 :
	public TEOSCallbackWithNested1Param3<CallbackFuncType, CallbackType, OwningType, Nested1CallbackFuncType, Nested1CallbackType, Nested1ReturnType>
{
public:
	TEOSCallbackWithNested2ForNested1Param3(TWeakPtr<OwningType> InOwner)
		: TEOSCallbackWithNested1Param3<CallbackFuncType, CallbackType, OwningType, Nested1CallbackFuncType, Nested1CallbackType, Nested1ReturnType>(InOwner)
	{
	}
	virtual ~TEOSCallbackWithNested2ForNested1Param3() = default;


	Nested2CallbackFuncType GetNested2CallbackPtr()
	{
		return &Nested2CallbackImpl;
	}

	void SetNested2CallbackLambda(TFunction<void(const Nested2CallbackType*)> InLambda)
	{
		Nested2CallbackLambda = InLambda;
	}

private:
	TFunction<void(const Nested2CallbackType*)> Nested2CallbackLambda;

	static void EOS_CALL Nested2CallbackImpl(const Nested2CallbackType* Data)
	{
		check(IsInGameThread());
		TEOSCallbackWithNested2ForNested1Param3* CallbackThis = (TEOSCallbackWithNested2ForNested1Param3*)Data->ClientData;
		check(CallbackThis);

		if (CallbackThis->Owner.IsValid())
		{
			check(CallbackThis->CallbackLambda);
			CallbackThis->Nested2CallbackLambda(Data);
		}
	}
};

struct FSessionDetailsEOS : FNoncopyable
{
	EOS_HSessionDetails SessionDetailsHandle;

	FSessionDetailsEOS(EOS_HSessionDetails InSessionDetailsHandle)
		: SessionDetailsHandle(InSessionDetailsHandle)
	{
	}

	virtual ~FSessionDetailsEOS()
	{
		EOS_SessionDetails_Release(SessionDetailsHandle);
	}
};

struct FLobbyDetailsEOS : FNoncopyable
{
	EOS_HLobbyDetails LobbyDetailsHandle;

	FLobbyDetailsEOS(EOS_HLobbyDetails InLobbyDetailsHandle)
		: LobbyDetailsHandle(InLobbyDetailsHandle)
	{
	}

	virtual ~FLobbyDetailsEOS()
	{
		EOS_LobbyDetails_Release(LobbyDetailsHandle);
	}
};

/**
 * Implementation of session information
 */
class FOnlineSessionInfoEOS :
	public FOnlineSessionInfo
{
protected:
	/** Hidden on purpose */
	FOnlineSessionInfoEOS& operator=(const FOnlineSessionInfoEOS& Src)
	{
		return *this;
	}

PACKAGE_SCOPE:
	/** Constructor */
	FOnlineSessionInfoEOS() = default;
	
	FOnlineSessionInfoEOS(const FOnlineSessionInfoEOS&) = default;

	static FOnlineSessionInfoEOS Create(FUniqueNetIdStringRef UniqueNetId);
	static FOnlineSessionInfoEOS Create(FUniqueNetIdStringRef UniqueNetId, const TSharedPtr<FSessionDetailsEOS>& SessionHandle);
	static FOnlineSessionInfoEOS Create(FUniqueNetIdStringRef UniqueNetId, const TSharedPtr<FLobbyDetailsEOS>& LobbyHandle);

	/**
	 * Initialize LAN session
	 */
	void InitLAN(FOnlineSubsystemEOS* Subsystem);

	/** The ip & port that the host is listening on (valid for LAN/GameServer) */
	TSharedPtr<class FInternetAddr> HostAddr = nullptr;
	/** Unique Id for this session */
	FUniqueNetIdStringRef SessionId = FUniqueNetIdString::EmptyId();
	/** EOS session handle. The same handle can be shared between a local session and a search result. The struct type will call the release API automatically upon destruction */
	TSharedPtr<FSessionDetailsEOS> SessionHandle = nullptr;
	/** EOS lobby handle. The same handle can be shared between a local session and a search result. The struct type will call the release API automatically upon destruction */
	TSharedPtr<FLobbyDetailsEOS> LobbyHandle = nullptr;

public:
	virtual ~FOnlineSessionInfoEOS() = default;

	bool operator==(const FOnlineSessionInfoEOS& Other) const
	{
		return false;
	}
	virtual const uint8* GetBytes() const override
	{
		return nullptr;
	}
	virtual int32 GetSize() const override
	{
		return sizeof(uint64) + sizeof(TSharedPtr<class FInternetAddr>);
	}
	virtual bool IsValid() const override
	{
		// LAN case
		return HostAddr.IsValid() && HostAddr->IsValid();
	}
	virtual FString ToString() const override
	{
		return SessionId->ToString();
	}
	virtual FString ToDebugString() const override
	{
		return FString::Printf(TEXT("HostIP: %s SessionId: %s"),
			HostAddr.IsValid() ? *HostAddr->ToString(true) : TEXT("INVALID"),
			*SessionId->ToDebugString());
	}
	virtual const FUniqueNetId& GetSessionId() const override
	{
		return *SessionId;
	}
};

#endif
