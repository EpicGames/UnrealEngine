// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineTitleFileEOS.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemEOS.h"
#include "OnlineSubsystemEOSPrivate.h"
#include "OnlineSubsystemEOSTypes.h"
#include "UserManagerEOS.h"
#include "EOSSettings.h"

#if WITH_EOS_SDK
#include "eos_titlestorage.h"

typedef TEOSCallback<EOS_TitleStorage_OnDeleteCacheCompleteCallback, EOS_TitleStorage_DeleteCacheCallbackInfo, FOnlineTitleFileEOS> FDeleteCacheCompleteCallback;

typedef TEOSCallback<EOS_TitleStorage_OnQueryFileListCompleteCallback, EOS_TitleStorage_QueryFileListCallbackInfo, FOnlineTitleFileEOS> FQueryFileListCallback;

typedef TEOSCallbackWithNested2<EOS_TitleStorage_OnReadFileCompleteCallback, EOS_TitleStorage_ReadFileCallbackInfo, FOnlineTitleFileEOS,
	EOS_TitleStorage_OnReadFileDataCallback, EOS_TitleStorage_ReadFileDataCallbackInfo, EOS_TitleStorage_EReadResult,
	EOS_TitleStorage_OnFileTransferProgressCallback, EOS_TitleStorage_FileTransferProgressCallbackInfo
> FReadTitleFileCompleteCallback;

void FEOSTitleFile::Unload()
{
	if (bIsLoaded)
	{
		Contents.Empty();
		bIsLoaded = false;
	}
	else if (bInProgress)
	{
		UE_LOG_ONLINE_TITLEFILE(Warning, TEXT("Unload() file (%s) is being read"), *Filename);
	}
}

bool FOnlineTitleFileEOS::GetFileContents(const FString& FileName, TArray<uint8>& FileContents)
{
	FEOSTitleFile* File = FileSet.Find(FileName);
	if (File != nullptr)
	{
		if (File->bIsLoaded)
		{
			FileContents = File->Contents;
			UE_LOG_ONLINE_TITLEFILE(Verbose, TEXT("GetFileContents() got data for (%s), %d bytes"), *FileName, FileContents.Num());
			return true;
		}
		else
		{
			// Are we still downloading?
			if (File->bInProgress)
			{
				UE_LOG_ONLINE_TITLEFILE(Warning, TEXT("GetFileContents() file (%s) is being read"), *FileName);
			}
			// Contents have been cleared
			else
			{
				UE_LOG_ONLINE_TITLEFILE(Warning, TEXT("GetFileContents() file (%s) has not been loaded"), *FileName);
			}
		}
	}
	else
	{
		UE_LOG_ONLINE_TITLEFILE(Warning, TEXT("GetFileContents() unknown file (%s)"), *FileName);
	}

	return false;
}

bool FOnlineTitleFileEOS::ClearFiles()
{
	for (TPair<FString, FEOSTitleFile>& TitleFile : FileSet)
	{
		FEOSTitleFile& File = TitleFile.Value;
		File.Unload();
	}
	return true;
}

bool FOnlineTitleFileEOS::ClearFile(const FString& FileName)
{
	FEOSTitleFile* File = FileSet.Find(FileName);
	if (File != nullptr)
	{
		File->Unload();
		return true;
	}
	return false;
}

void FOnlineTitleFileEOS::DeleteCachedFiles(bool bSkipEnumerated)
{
	if (bSkipEnumerated)
	{
		UE_LOG_ONLINE_TITLEFILE(Warning, TEXT("DeleteCachedFiles() bSkipEnumerated option ignored"));
	}

	EOS_TitleStorage_DeleteCacheOptions DeleteCacheOptions = { };
	DeleteCacheOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_TITLESTORAGE_DELETECACHEOPTIONS_API_LATEST, 1);
	DeleteCacheOptions.LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(EOSSubsystem->UserManager->GetDefaultLocalUser());	// Get a local user if one is available, but this is not required

	FDeleteCacheCompleteCallback* CallbackObj = new FDeleteCacheCompleteCallback(FOnlineTitleFileEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this](const EOS_TitleStorage_DeleteCacheCallbackInfo* Data)
	{
		bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
		if (bWasSuccessful)
		{
			UE_LOG_ONLINE_TITLEFILE(Verbose, TEXT("Deleted cached files"));
		}
		else
		{
			UE_LOG_ONLINE_TITLEFILE(Error, TEXT("EOS_TitleStorage_DeleteCache() failed with error code (%s)"), *LexToString(Data->ResultCode));
		}
	};

	EOS_TitleStorage_DeleteCache(EOSSubsystem->TitleStorageHandle, &DeleteCacheOptions, CallbackObj, CallbackObj->GetCallbackPtr());
}

bool FOnlineTitleFileEOS::EnumerateFiles(const FPagedQuery& Page)
{
	FString ErrorStr;
	bool bStarted = true;
	TArray<FString> TitleStorageTags = UEOSSettings::GetSettings().TitleStorageTags;

	if (TitleStorageTags.Num() == 0)
	{
		ErrorStr = TEXT("No tags defined");
		bStarted = false;
	}
	else if (Page.Start >= TitleStorageTags.Num())
	{
		ErrorStr = FString::Printf(TEXT("Unknown tag page %d"), Page.Start);
		bStarted = false;
	}

	if (!bStarted)
	{
		EOSSubsystem->ExecuteNextTick([this, ErrorStr]()
		{
			UE_LOG_ONLINE_TITLEFILE(Error, TEXT("EnumerateFiles() %s"), *ErrorStr);
			TriggerOnEnumerateFilesCompleteDelegates(false, *ErrorStr);
		});
		return true;
	}

	// Find all tags defined for the start page
	FString& PageTagsRaw = TitleStorageTags[Page.Start];
	const TCHAR* TagDelims[1] = { TEXT("+") };
	TArray<FString> PageTags;
	PageTagsRaw.ParseIntoArray(PageTags, TagDelims, 1, false);

	TArray<FTCHARToUTF8> PageTagsBuffer;
	TArray<const char*> AnsiTags;
	PageTagsBuffer.Reserve(PageTags.Num());
	AnsiTags.Reserve(PageTags.Num());

	for (int TagIndex = 0; TagIndex < PageTags.Num(); ++TagIndex)
	{
		const FString& PageTag = PageTags[TagIndex];
		FTCHARToUTF8& Converter = PageTagsBuffer.Emplace_GetRef(*PageTag);
		AnsiTags.Emplace(Converter.Get());
		UE_LOG_ONLINE_TITLEFILE(VeryVerbose, TEXT("EnumerateFiles() using tag (%s)"), *PageTag);
	}
	UE_LOG_ONLINE_TITLEFILE(VeryVerbose, TEXT("EnumerateFiles() found %d tags"), AnsiTags.Num());


	EOS_TitleStorage_QueryFileListOptions QueryFileListOptions = { };
	QueryFileListOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_TITLESTORAGE_QUERYFILELISTOPTIONS_API_LATEST, 1);
	QueryFileListOptions.LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(EOSSubsystem->UserManager->GetDefaultLocalUser());	// Get a local user if one is available, but this is not required
	QueryFileListOptions.ListOfTags = const_cast<const char**>(AnsiTags.GetData());
	QueryFileListOptions.ListOfTagsCount = AnsiTags.Num();

	FQueryFileListCallback* CallbackObj = new FQueryFileListCallback(FOnlineTitleFileEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this](const EOS_TitleStorage_QueryFileListCallbackInfo* Data)
	{
		FString ErrorStr;
		bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
		if (bWasSuccessful)
		{
			uint32 FileCount = Data->FileCount;
			UE_LOG_ONLINE_TITLEFILE(Verbose, TEXT("Found %d files"), FileCount);

			QueryFileSet.Empty();

			for (uint32 Index = 0; Index < FileCount; ++Index)
			{
				EOS_TitleStorage_CopyFileMetadataAtIndexOptions CopyFileMetadataAtIndexOptions = { };
				CopyFileMetadataAtIndexOptions.ApiVersion = 1;
				UE_EOS_CHECK_API_MISMATCH(EOS_TITLESTORAGE_COPYFILEMETADATAATINDEXOPTIONS_API_LATEST, 1);
				CopyFileMetadataAtIndexOptions.LocalUserId = Data->LocalUserId;
				CopyFileMetadataAtIndexOptions.Index = Index;

				EOS_TitleStorage_FileMetadata* FileMetadata = nullptr;

				EOS_EResult Result = EOS_TitleStorage_CopyFileMetadataAtIndex(EOSSubsystem->TitleStorageHandle, &CopyFileMetadataAtIndexOptions, &FileMetadata);
				if (Result == EOS_EResult::EOS_Success)
				{
					if (FileMetadata && FileMetadata->Filename)
					{
						QueryFileSet.Emplace(FCloudFileHeader(ANSI_TO_TCHAR(FileMetadata->Filename), ANSI_TO_TCHAR(FileMetadata->Filename), FileMetadata->FileSizeBytes));
						UE_LOG_ONLINE_TITLEFILE(VeryVerbose, TEXT("Metadata for (%s), size %d"), ANSI_TO_TCHAR(FileMetadata->Filename), FileMetadata->FileSizeBytes);
					}
					EOS_TitleStorage_FileMetadata_Release(FileMetadata);
				}
				else
				{
					ErrorStr = LexToString(Result);
					UE_LOG_ONLINE_TITLEFILE(Error, TEXT("EOS_TitleStorage_CopyFileMetadataAtIndex() failed with error code (%s)"), *ErrorStr);
				}
			}
		}
		else
		{
			ErrorStr = LexToString(Data->ResultCode);
			UE_LOG_ONLINE_TITLEFILE(Error, TEXT("EOS_TitleStorage_QueryFileList() failed with error code (%s)"), *ErrorStr);
		}

		TriggerOnEnumerateFilesCompleteDelegates(bWasSuccessful, *ErrorStr);
	};

	EOS_TitleStorage_QueryFileList(EOSSubsystem->TitleStorageHandle, &QueryFileListOptions, CallbackObj, CallbackObj->GetCallbackPtr());
	return true;
}

// Get the results from the last completed EnumerateFiles request. This data has the potential to become stale over time.
void FOnlineTitleFileEOS::GetFileList(TArray<FCloudFileHeader>& Files)
{
	Files = QueryFileSet;
}

bool FOnlineTitleFileEOS::ReadFile(const FString& FileName)
{
	// If this file is already downloading, dont try to start it again.
	// Do allow the re-reading of files previously cleared, or already loaded files that may have changed in the cloud.
	FEOSTitleFile *ExistingTitleFile = FileSet.Find(FileName);
	if (ExistingTitleFile != nullptr && ExistingTitleFile->bInProgress)
	{
		EOSSubsystem->ExecuteNextTick([this, FileName]()
		{
			UE_LOG_ONLINE_TITLEFILE(Warning, TEXT("ReadFile() already being read (%s)"), *FileName);
			TriggerOnReadFileCompleteDelegates(false, FileName);
		});
		return true;
	}

	FReadTitleFileCompleteCallback* CallbackObj = new FReadTitleFileCompleteCallback(FOnlineTitleFileEOSWeakPtr(AsShared()));

	CallbackObj->SetNested1CallbackLambda([this](const EOS_TitleStorage_ReadFileDataCallbackInfo* Data)
	{
		UE_LOG_ONLINE_TITLEFILE(VeryVerbose, TEXT("Read file data (%s) %d bytes"), ANSI_TO_TCHAR(Data->Filename), Data->DataChunkLengthBytes);
		FEOSTitleFile* TitleFile = FileSet.Find(FString(ANSI_TO_TCHAR(Data->Filename)));
		if (TitleFile != nullptr)
		{
			check(TitleFile->bInProgress);
			// Is this is the first chunk of data we have received for this file?
			if (TitleFile->ContentSize == 0 && TitleFile->ContentIndex == 0)
			{
				// Store the actual size of the file being read
				TitleFile->ContentSize = Data->TotalFileSizeBytes;

				// Is the file being read empty?
				if (TitleFile->ContentSize == 0)
				{
					return EOS_TitleStorage_EReadResult::EOS_TS_RR_ContinueReading;
				}

				TitleFile->Contents.AddUninitialized(Data->TotalFileSizeBytes);
			}

			if (TitleFile->ContentIndex + Data->DataChunkLengthBytes <= TitleFile->ContentSize)
			{
				check(Data->DataChunkLengthBytes > 0);
				FMemory::Memcpy(TitleFile->Contents.GetData()+ TitleFile->ContentIndex, Data->DataChunk, Data->DataChunkLengthBytes);
				TitleFile->ContentIndex += Data->DataChunkLengthBytes;
				return EOS_TitleStorage_EReadResult::EOS_TS_RR_ContinueReading;
			}
			else
			{
				UE_LOG_ONLINE_TITLEFILE(Warning, TEXT("EOS_TitleStorage_ReadFile() read size exceeded specified file size (%s)"), ANSI_TO_TCHAR(Data->Filename));
				return EOS_TitleStorage_EReadResult::EOS_TS_RR_FailRequest;
			}
		}
		else
		{
			UE_LOG_ONLINE_TITLEFILE(Warning, TEXT("EOS_TitleStorage_ReadFile() unknown file cancelling transfer request (%s)"), ANSI_TO_TCHAR(Data->Filename));
		}

		return EOS_TitleStorage_EReadResult::EOS_TS_RR_CancelRequest;
	});

	CallbackObj->SetNested2CallbackLambda([this](const EOS_TitleStorage_FileTransferProgressCallbackInfo* Data)
	{
		UE_LOG_ONLINE_TITLEFILE(VeryVerbose, TEXT("File transfer progress (%s) %d bytes"), ANSI_TO_TCHAR(Data->Filename), Data->BytesTransferred);
		TriggerOnReadFileProgressDelegates(FString(ANSI_TO_TCHAR(Data->Filename)), Data->BytesTransferred);
	});

	CallbackObj->CallbackLambda = [this](const EOS_TitleStorage_ReadFileCallbackInfo* Data)
	{
		bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;

		FEOSTitleFile* TitleFile = FileSet.Find(FString(ANSI_TO_TCHAR(Data->Filename)));
		if (TitleFile != nullptr)
		{
			if (TitleFile->FileTransferRequest != nullptr)
			{
				EOS_TitleStorageFileTransferRequest_Release(TitleFile->FileTransferRequest);
				TitleFile->FileTransferRequest = nullptr;
			}

			if (bWasSuccessful)
			{
				TitleFile->bIsLoaded = true;
				TitleFile->bInProgress = false;
				UE_LOG_ONLINE_TITLEFILE(Verbose, TEXT("Read (%s), size %zu"), *TitleFile->Filename, TitleFile->ContentSize);
			}
			else
			{
				// If we fail to complete reading the file, discard it from the known files
				FileSet.Remove(FString(ANSI_TO_TCHAR(Data->Filename)));

				UE_LOG_ONLINE_TITLEFILE(Error, TEXT("EOS_TitleStorage_ReadFile() failed with error code (%s)"), *LexToString(Data->ResultCode));
			}
		}
		else
		{
			bWasSuccessful = false;
			UE_LOG_ONLINE_TITLEFILE(Warning, TEXT("EOS_TitleStorage_ReadFile() unknown transfer request (%s)"), ANSI_TO_TCHAR(Data->Filename));
		}

		TriggerOnReadFileCompleteDelegates(bWasSuccessful, FString(ANSI_TO_TCHAR(Data->Filename)));
	};

	FTCHARToUTF8 FileNameConverter(*FileName);
	const char* AnsiFileName = FileNameConverter.Get();

	int32 ReadChunkSize = UEOSSettings::GetSettings().TitleStorageReadChunkLength;
	if (ReadChunkSize <= 0)
	{
		UE_LOG_ONLINE_TITLEFILE(Warning, TEXT("ReadFile() invalid size TitleStorageReadChunkLength %d"), ReadChunkSize);
		ReadChunkSize = 16 * 1024;
	}

	EOS_TitleStorage_ReadFileOptions ReadFileOptions = { };
	ReadFileOptions.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_TITLESTORAGE_READFILEOPTIONS_API_LATEST, 2);
	ReadFileOptions.LocalUserId = EOSSubsystem->UserManager->GetLocalProductUserId(EOSSubsystem->UserManager->GetDefaultLocalUser());	// Get a local user if one is available, but this is not required
	ReadFileOptions.Filename = AnsiFileName;
	ReadFileOptions.ReadChunkLengthBytes = (uint32)ReadChunkSize;
	ReadFileOptions.ReadFileDataCallback = CallbackObj->GetNested1CallbackPtr();
	ReadFileOptions.FileTransferProgressCallback = CallbackObj->GetNested2CallbackPtr();

	UE_LOG_ONLINE_TITLEFILE(Verbose, TEXT("ReadFile() reading (%s)"), *FileName);
	EOS_HTitleStorageFileTransferRequest FileTransferRequest = EOS_TitleStorage_ReadFile(EOSSubsystem->TitleStorageHandle, &ReadFileOptions, CallbackObj, CallbackObj->GetCallbackPtr());

	bool bStarted = (FileTransferRequest != nullptr);
	if (bStarted)
	{
		FEOSTitleFile TitleFile;
		TitleFile.Filename = FileName;
		TitleFile.FileTransferRequest = FileTransferRequest;
		TitleFile.bInProgress = true;
		FileSet.FindOrAdd(FileName) = MoveTemp(TitleFile);			// Replace the last title file, or create a new entry
	}
	else
	{
		EOSSubsystem->ExecuteNextTick([this, FileName]()
		{
			UE_LOG_ONLINE_TITLEFILE(Error, TEXT("ReadFile() failed to create a transfer request (%s)"), *FileName);
			TriggerOnReadFileCompleteDelegates(false, FileName);
		});
	}

	return true;
}

FDelegateHandle OnEnumerateFilesCompleteDelegateHandle;
FDelegateHandle OnReadFileProgressDelegateHandle;
FDelegateHandle OnReadFileCompleteDelegateHandle;

bool FOnlineTitleFileEOS::HandleTitleFileExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("ENUMERATE")))
	{
		FString PageString;
		int32 Page = 0;

		if (FParse::Token(Cmd, PageString, false))
		{
			Page = FCString::Atoi(*PageString);
		}

		OnEnumerateFilesCompleteDelegateHandle = AddOnEnumerateFilesCompleteDelegate_Handle(FOnEnumerateFilesCompleteDelegate::CreateLambda([this](bool bWasSuccessful, const FString& Error)
		{
			UE_LOG_ONLINE(Log, TEXT("EnumerateFiles: %s with error (%s)"), bWasSuccessful ? TEXT("succeeded") : TEXT("failed"), *Error);

			ClearOnEnumerateFilesCompleteDelegate_Handle(OnEnumerateFilesCompleteDelegateHandle);
		}));

		EnumerateFiles(FPagedQuery(Page));
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("GETFILELIST")))
	{
		TArray<FCloudFileHeader> Files;

		GetFileList(Files);
		UE_LOG_ONLINE(Log, TEXT("GetFilesList: %d files last enumerated"), Files.Num());

		for (int32 Index = 0; Index < Files.Num(); ++Index)
		{
			UE_LOG_ONLINE(Log, TEXT("GetFilesList: File (%s), encrypted size %d bytes"), *Files[Index].FileName, Files[Index].FileSize);
		}

		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("READFILE")))
	{
		FString FileName = FParse::Token(Cmd, false);
		if (FileName.IsEmpty())
		{
			UE_LOG_ONLINE(Log, TEXT("ReadFile: Missing filename"));
			return true;
		}

		// Note: These delegates will trigger for all files being read, if multiple requests are inflight
		OnReadFileProgressDelegateHandle = AddOnReadFileProgressDelegate_Handle(FOnReadFileProgressDelegate::CreateLambda([this](const FString& FileName, uint64 NumBytes)
		{
			UE_LOG_ONLINE(Log, TEXT("ReadFile: Progress (%s) %llu bytes read"), *FileName, NumBytes);
		}));

		OnReadFileCompleteDelegateHandle = AddOnReadFileCompleteDelegate_Handle(FOnReadFileCompleteDelegate::CreateLambda([this](bool bWasSuccessful, const FString& Error)
		{
			UE_LOG_ONLINE(Log, TEXT("ReadFile: %s with error (%s)"), bWasSuccessful ? TEXT("succeeded") : TEXT("failed"), *Error);

			ClearOnReadFileProgressDelegate_Handle(OnReadFileProgressDelegateHandle);
			ClearOnReadFileCompleteDelegate_Handle(OnReadFileCompleteDelegateHandle);
		}));

		ReadFile(FileName);
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("GETFILECONTENTS")))
	{
		FString FileName = FParse::Token(Cmd, false);
		if (FileName.IsEmpty())
		{
			UE_LOG_ONLINE(Log, TEXT("GetFileContents: Missing filename"));
			return true;
		}

		TArray<uint8> FileContents;

		if (GetFileContents(FileName, FileContents))
		{
			UE_LOG_ONLINE(Log, TEXT("GetFileContents: Got contents for (%s), size %d bytes"), *FileName, FileContents.Num());
		}
		else
		{
			UE_LOG_ONLINE(Log, TEXT("GetFilesContents: Unknown filename or not loaded (%s)"), *FileName);
		}

		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("CLEARFILES")))
	{
		ClearFiles();
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("CLEARFILE")))
	{
		FString FileName = FParse::Token(Cmd, false);
		if (FileName.IsEmpty())
		{
			UE_LOG_ONLINE(Log, TEXT("ClearFile: Missing filename"));
			return true;
		}

		if (ClearFile(FileName))
		{
			UE_LOG_ONLINE(Log, TEXT("ClearFile: Cleared (%s)"), *FileName);
		}
		else
		{
			UE_LOG_ONLINE(Log, TEXT("ClearFile: Unknown filename or not loaded (%s)"), *FileName);
		}

		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("DELETECACHEDFILES")))
	{
		DeleteCachedFiles(false);
		return true;
	}
	return false;
}

#endif
