// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Online/CoreOnline.h"
#include "Interfaces/OnlineTitleFileInterface.h"
#include "OnlineSubsystemEOSTypes.h"

class FOnlineSubsystemEOS;

#if WITH_EOS_SDK
#include "eos_titlestorage_types.h"

struct FEOSTitleFile
{
	TArray<uint8> Contents;
	size_t ContentSize;
	size_t ContentIndex;
	bool bIsLoaded;
	bool bInProgress;
	FString Filename;
	EOS_HTitleStorageFileTransferRequest FileTransferRequest;		// TODO: Mark.Fitt this does not auto release

	FEOSTitleFile() : ContentSize(0), ContentIndex(0), bIsLoaded(false), bInProgress(false)
	{
	}

	void Unload();
};

using FTitleFileCollection = TMap<FString, FEOSTitleFile>;

class FOnlineTitleFileEOS
	: public IOnlineTitleFile, public TSharedFromThis<FOnlineTitleFileEOS, ESPMode::ThreadSafe>
{
public:
	FOnlineTitleFileEOS() = delete;
	virtual ~FOnlineTitleFileEOS() = default;

//~IOnlineTitleFile
	virtual bool GetFileContents(const FString& FileName, TArray<uint8>& FileContents) override;
	virtual bool ClearFiles() override;
	virtual bool ClearFile(const FString& FileName) override;
	virtual void DeleteCachedFiles(bool bSkipEnumerated) override;
	virtual bool EnumerateFiles(const FPagedQuery& Page = FPagedQuery()) override;
	virtual void GetFileList(TArray<FCloudFileHeader>& Files) override;
	virtual bool ReadFile(const FString& FileName) override;
//~IOnlineTitleFile

	FOnlineTitleFileEOS(FOnlineSubsystemEOS* InSubsystem)
		: EOSSubsystem(InSubsystem)
	{
	}

	bool HandleTitleFileExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);

protected:
	FOnlineSubsystemEOS* EOSSubsystem;

private:
	/** Results of the last file enumeration */
	TArray<FCloudFileHeader> QueryFileSet;
	/** The list of available files, indexed by filename that have been or are loaded */
	FTitleFileCollection FileSet;
};

typedef TSharedPtr<FOnlineTitleFileEOS, ESPMode::ThreadSafe> FOnlineTitleFileEOSPtr;
typedef TWeakPtr<FOnlineTitleFileEOS, ESPMode::ThreadSafe> FOnlineTitleFileEOSWeakPtr;

#endif