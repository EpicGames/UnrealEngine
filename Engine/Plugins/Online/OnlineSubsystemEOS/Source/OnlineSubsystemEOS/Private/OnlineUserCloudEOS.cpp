// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineUserCloudEOS.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemEOS.h"
#include "OnlineSubsystemEOSPrivate.h"
#include "UserManagerEOS.h"
#include "EOSSettings.h"

#if WITH_EOS_SDK
#include "eos_playerdatastorage.h"

typedef TEOSCallback<EOS_PlayerDataStorage_OnQueryFileListCompleteCallback, EOS_PlayerDataStorage_QueryFileListCallbackInfo, FOnlineUserCloudEOS> FOnQueryFileListCallback;

typedef TEOSCallbackWithNested2ForNested1Param3<EOS_PlayerDataStorage_OnWriteFileCompleteCallback, EOS_PlayerDataStorage_WriteFileCallbackInfo, FOnlineUserCloudEOS,
	EOS_PlayerDataStorage_OnWriteFileDataCallback, EOS_PlayerDataStorage_WriteFileDataCallbackInfo, EOS_PlayerDataStorage_EWriteResult,
	EOS_PlayerDataStorage_OnFileTransferProgressCallback, EOS_PlayerDataStorage_FileTransferProgressCallbackInfo
> FWriteUserFileCompleteCallback;

typedef TEOSCallbackWithNested2<EOS_PlayerDataStorage_OnReadFileCompleteCallback, EOS_PlayerDataStorage_ReadFileCallbackInfo, FOnlineUserCloudEOS,
	EOS_PlayerDataStorage_OnReadFileDataCallback, EOS_PlayerDataStorage_ReadFileDataCallbackInfo, EOS_PlayerDataStorage_EReadResult,
	EOS_PlayerDataStorage_OnFileTransferProgressCallback, EOS_PlayerDataStorage_FileTransferProgressCallbackInfo
> FReadUserFileCompleteCallback;

typedef TEOSCallback<EOS_PlayerDataStorage_OnDeleteFileCompleteCallback, EOS_PlayerDataStorage_DeleteFileCallbackInfo, FOnlineUserCloudEOS> FOnDeleteFileCallback;

void FEOSUserCloudFile::Unload()
{
	if (bIsLoaded)
	{
		Contents.Empty();
		bIsLoaded = false;
	}
	else if (bInProgress)
	{
		UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FEOSUserCloudFile::Unload] Could not unload. File %s is being read or written"), *Filename);
	}
}

bool FOnlineUserCloudEOS::GetFileContents(const FUniqueNetId& UserId, const FString& FileName, TArray<uint8>& FileContents)
{
	FUniqueNetIdPtr UniqueNetId = EOSSubsystem->UserManager->GetUniquePlayerId(EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(UserId));
	if (!UniqueNetId.IsValid())
	{
		UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::GetFileContents] Unable to get file contents. User %s is not a local user"), *UserId.ToString());
		return false;
	}

	bool bResult = false;

	FUserCloudFileCollection* UserCloudFileCollection = FileSetsPerUser.Find(UniqueNetId.ToSharedRef());
	if (UserCloudFileCollection != nullptr)
	{
		FEOSUserCloudFile* UserCloudFile = UserCloudFileCollection->Find(FileName);
		if (UserCloudFile != nullptr)
		{
			if (UserCloudFile->bIsLoaded)
			{
				FileContents = UserCloudFile->Contents;

				UE_LOG_ONLINE_CLOUD(Verbose, TEXT("[FOnlineUserCloudEOS::GetFileContents] Got data for file %s with a size of %d bytes"), *FileName, FileContents.Num());

				bResult = true;
			}
			else
			{
				// Are we still downloading?
				if (UserCloudFile->bInProgress)
				{
					UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::GetFileContents] Unable to get file contents. File %s is being read or written"), *FileName);
				}
				// Contents have been cleared
				else
				{
					UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::GetFileContents] Unable to get file contents. File %s has not been loaded"), *FileName);
				}
			}
		}
		else
		{
			UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::GetFileContents] Unable to get file contents. Unknown file %s"), *FileName);
		}
	}
	else
	{
		UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::GetFileContents] Unable to get file contents. Unknown user %s"), *UserId.ToString());
	}

	return bResult;
}

bool FOnlineUserCloudEOS::ClearFiles(const FUniqueNetId& UserId)
{
	FUniqueNetIdPtr UniqueNetId = EOSSubsystem->UserManager->GetUniquePlayerId(EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(UserId));
	if (!UniqueNetId.IsValid())
	{
		UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::ClearFiles] Unable to get clear files. User %s is not a local user"), *UserId.ToString());
		return false;
	}

	bool bResult = true;

	// First we check if all the files for this user can be deleted, without actually deleting any.
	FUserCloudFileCollection* UserCloudFileCollection = FileSetsPerUser.Find(UniqueNetId.ToSharedRef());
	if (UserCloudFileCollection != nullptr)
	{
		for (TPair<FString, FEOSUserCloudFile>& Entry : *UserCloudFileCollection)
		{
			if (Entry.Value.bInProgress)
			{
				UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::ClearFiles] Unable to clear files. File %s for user %s is in the process of being read or written"), *Entry.Key, *UserId.ToString());
				bResult = false;
				break;
			}
		}
	}
	else
	{
		UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::ClearFiles] Unable to clear files. Unknown user %s"), *UserId.ToString());
		bResult = false;
	}

	// Then, if all files are ok for clearing, we do so
	if (bResult)
	{
		FileSetsPerUser.Remove(UniqueNetId.ToSharedRef());

		UE_LOG_ONLINE_CLOUD(Verbose, TEXT("[FOnlineUserCloudEOS::ClearFiles] Cleared all files for user %s"), *UserId.ToString());
	}
	
	return bResult;
}

bool FOnlineUserCloudEOS::ClearFile(const FUniqueNetId& UserId, const FString& FileName)
{
	FUniqueNetIdPtr UniqueNetId = EOSSubsystem->UserManager->GetUniquePlayerId(EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(UserId));
	if (!UniqueNetId.IsValid())
	{
		UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::ClearFile] Unable to get clear file. User %s is not a local user"), *UserId.ToString());
		return false;
	}

	bool bResult = false;

	FUserCloudFileCollection* UserCloudFileCollection = FileSetsPerUser.Find(UniqueNetId.ToSharedRef());
	if (UserCloudFileCollection != nullptr)
	{
		FEOSUserCloudFile* UserCloudFile = UserCloudFileCollection->Find(FileName);
		if (UserCloudFile != nullptr)
		{
			if (UserCloudFile->bInProgress)
			{
				UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::ClearFile] Unable to clear file. File %s for user %s is in the process of being read or written"), *FileName, *UserId.ToString());
			}
			else
			{
				UserCloudFileCollection->Remove(FileName);

				UE_LOG_ONLINE_CLOUD(Verbose, TEXT("[FOnlineUserCloudEOS::ClearFile] Cleared file %s for user %s"), *FileName, *UserId.ToString());

				bResult = true;
			}
		}
		else
		{
			UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::ClearFile] Unable to clear file. Unknown file %s"), *FileName);
		}
	}
	else
	{
		UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::ClearFile] Unable to clear file. Unknown user %s"), *UserId.ToString());
	}

	return bResult;
}

void FOnlineUserCloudEOS::EnumerateUserFiles(const FUniqueNetId& UserId)
{
	const FUniqueNetIdEOS& UserEOSId = FUniqueNetIdEOS::Cast(UserId);
	const EOS_ProductUserId LocalUserId = UserEOSId.GetProductUserId();

	if (LocalUserId == nullptr)
	{
		EOSSubsystem->ExecuteNextTick([this, UserIdRef = UserId.AsShared()]() {
				UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::EnumerateUserFiles] Could not retrieve EOS_ProductUserId for user %s"), *UserIdRef->ToString());
				TriggerOnEnumerateUserFilesCompleteDelegates(false, *UserIdRef);
			});

		return;
	}

	EOS_PlayerDataStorage_QueryFileListOptions Options = {};
	Options.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_PLAYERDATASTORAGE_QUERYFILELISTOPTIONS_API_LATEST, 2);
	Options.LocalUserId = LocalUserId;

	FOnQueryFileListCallback* CallbackObj = new FOnQueryFileListCallback(FOnlineUserCloudEOSWeakPtr(AsShared()));
	CallbackObj->CallbackLambda = [this](const EOS_PlayerDataStorage_QueryFileListCallbackInfo* Data)
	{
		bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success || Data->ResultCode == EOS_EResult::EOS_NotFound; // If the user doesn't have any files yet, we will get a NotFound error, but the query was valid

		FUniqueNetIdEOSPtr UniqueNetIdPtr = EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(Data->LocalUserId);
		if (UniqueNetIdPtr.IsValid())
		{
			if (bWasSuccessful)
			{
				uint32 FileCount = Data->FileCount;
				UE_LOG_ONLINE_CLOUD(Verbose, TEXT("[FOnlineUserCloudEOS::EnumerateUserFiles] Found %d files for user %s"), FileCount, *UniqueNetIdPtr->ToString());

				TArray<FCloudFileHeader>& QueryFileSet = QueryFileSetsPerUser.FindOrAdd(UniqueNetIdPtr.ToSharedRef());
				QueryFileSet.Empty();

				for (uint32 Index = 0; Index < FileCount; ++Index)
				{
					EOS_PlayerDataStorage_CopyFileMetadataAtIndexOptions CopyFileMetadataAtIndexOptions = { };
					CopyFileMetadataAtIndexOptions.ApiVersion = 1;
					UE_EOS_CHECK_API_MISMATCH(EOS_PLAYERDATASTORAGE_COPYFILEMETADATAATINDEXOPTIONS_API_LATEST, 1);
					CopyFileMetadataAtIndexOptions.LocalUserId = Data->LocalUserId;
					CopyFileMetadataAtIndexOptions.Index = Index;

					EOS_PlayerDataStorage_FileMetadata* FileMetadata = nullptr;
					EOS_EResult Result = EOS_PlayerDataStorage_CopyFileMetadataAtIndex(EOSSubsystem->PlayerDataStorageHandle, &CopyFileMetadataAtIndexOptions, &FileMetadata);
					if (Result == EOS_EResult::EOS_Success)
					{
						if (FileMetadata && FileMetadata->Filename)
						{
							QueryFileSet.Emplace(FCloudFileHeader(ANSI_TO_TCHAR(FileMetadata->Filename), ANSI_TO_TCHAR(FileMetadata->Filename), FileMetadata->FileSizeBytes));

							UE_LOG_ONLINE_CLOUD(VeryVerbose, TEXT("[FOnlineUserCloudEOS::EnumerateUserFiles] Cached metadata for file %s with size %d"), ANSI_TO_TCHAR(FileMetadata->Filename), FileMetadata->FileSizeBytes);
						}

						EOS_PlayerDataStorage_FileMetadata_Release(FileMetadata);
					}
					else
					{
						UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::EnumerateUserFiles] EOS_PlayerDataStorage_CopyFileMetadataAtIndex was not successful. Finished with error %s on file with index %d"), *LexToString(Result), Index);
					}
				}
			}
			else
			{
				UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::EnumerateUserFiles] EOS_PlayerDataStorage_QueryFileList was not successful. Finished with error %s"), *LexToString(Data->ResultCode));
			}
		}
		else
		{
			UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::EnumerateUserFiles] UniqueNetId not found for EOS_ProductUserId %lld"), Data->LocalUserId);
		}

		TriggerOnEnumerateUserFilesCompleteDelegates(bWasSuccessful, *UniqueNetIdPtr);
	};

	EOS_PlayerDataStorage_QueryFileList(EOSSubsystem->PlayerDataStorageHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
}

void FOnlineUserCloudEOS::GetUserFileList(const FUniqueNetId& UserId, TArray<FCloudFileHeader>& UserFiles)
{
	FUniqueNetIdPtr UniqueNetId = EOSSubsystem->UserManager->GetUniquePlayerId(EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(UserId));
	if (!UniqueNetId.IsValid())
	{
		UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::GetUserFileList] Unable to get user file list. User %s is not a local user"), *UserId.ToString());
		return;
	}

	if (const TArray<FCloudFileHeader>* const UserFileSets = QueryFileSetsPerUser.Find(UniqueNetId.ToSharedRef()))
	{
		UserFiles = *UserFileSets;
	}
	else
	{
		UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::GetUserFileList] User %s doesn't have any cached files."), *UserId.ToString());
	}
}

bool FOnlineUserCloudEOS::ReadUserFile(const FUniqueNetId& UserId, const FString& FileName)
{
	FUniqueNetIdPtr UniqueNetId = EOSSubsystem->UserManager->GetUniquePlayerId(EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(UserId));
	if (!UniqueNetId.IsValid())
	{
		EOSSubsystem->ExecuteNextTick([this, UserIdRef = UserId.AsShared(), FileName]()
			{
				UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::ReadUserFile] Unable to get read user file. User %s is not a local user"), *UserIdRef->ToString());
				TriggerOnReadUserFileCompleteDelegates(false, *UserIdRef, FileName);
			});		
		return true;
	}

	FUniqueNetIdRef SharedUserId = UniqueNetId.ToSharedRef();

	// If this file is already being read or written, don't try to start it again.
	FUserCloudFileCollection* UserCloudFileCollection = FileSetsPerUser.Find(SharedUserId);
	if (UserCloudFileCollection != nullptr)
	{
		FEOSUserCloudFile* UserCloudFile = UserCloudFileCollection->Find(FileName);
		if (UserCloudFile != nullptr && UserCloudFile->bInProgress)
		{
			EOSSubsystem->ExecuteNextTick([this, UserIdRef = UserId.AsShared(), FileName]()
				{
					UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::ReadUserFile] File %s for user %s is already in the process of being read or written"), *FileName, *UserIdRef->ToString());
					TriggerOnReadUserFileCompleteDelegates(false, *UserIdRef, FileName);
				});

			return true;
		}
	}

	// We prepare all the components for the Options object

	FTCHARToUTF8 FileNameUtf8(*FileName);

	int32 ReadChunkSize = UEOSSettings::GetSettings().TitleStorageReadChunkLength;
	if (ReadChunkSize <= 0)
	{
		UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::ReadUserFile] invalid size TitleStorageReadChunkLength %d. ReadChunkSize set to 16KB by default"), ReadChunkSize);
		ReadChunkSize = 16 * 1024;
	}

	FReadUserFileCompleteCallback* CallbackObj = new FReadUserFileCompleteCallback(FOnlineUserCloudEOSWeakPtr(AsShared()));

	CallbackObj->SetNested1CallbackLambda([this, SharedUserId, FileName](const EOS_PlayerDataStorage_ReadFileDataCallbackInfo* Data)
		{
			UE_LOG_ONLINE_CLOUD(VeryVerbose, TEXT("[FOnlineUserCloudEOS::ReadUserFile] Reading %d bytes of file %s's data"), Data->DataChunkLengthBytes, *FileName);

			FUserCloudFileCollection* UserCloudFileCollection = FileSetsPerUser.Find(SharedUserId);
			if (UserCloudFileCollection != nullptr)
			{
				FEOSUserCloudFile* UserCloudFile = UserCloudFileCollection->Find(FileName);
				if (UserCloudFile != nullptr)
				{
					check(UserCloudFile->bInProgress);

					// Is this is the first chunk of data we have received for this file?
					if (UserCloudFile->ContentSize == 0 && UserCloudFile->ContentIndex == 0)
					{
						// Store the actual size of the file being read
						UserCloudFile->ContentSize = Data->TotalFileSizeBytes;

						// Is the file being read empty?
						if (UserCloudFile->ContentSize == 0)
						{
							// If the file is empty, we return this value as we would do in the case when we have read the last bytes of a file. The OnReadComplete delegate will trigger immediately after.
							return EOS_PlayerDataStorage_EReadResult::EOS_RR_ContinueReading;
						}

						UserCloudFile->Contents.AddUninitialized(Data->TotalFileSizeBytes);
					}

					if (UserCloudFile->ContentIndex + Data->DataChunkLengthBytes <= UserCloudFile->ContentSize)
					{
						check(Data->DataChunkLengthBytes > 0);
						FMemory::Memcpy(UserCloudFile->Contents.GetData() + UserCloudFile->ContentIndex, Data->DataChunk, Data->DataChunkLengthBytes);
						UserCloudFile->ContentIndex += Data->DataChunkLengthBytes;
						return EOS_PlayerDataStorage_EReadResult::EOS_RR_ContinueReading;
					}
					else
					{
						UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::ReadUserFile] Read size exceeded specified file size for file %s"), *FileName);
						return EOS_PlayerDataStorage_EReadResult::EOS_RR_FailRequest;
					}
				}
				else
				{
					UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::ReadUserFile] Unknown file %s. Cancelling transfer request"), *FileName);
				}
			}
			else
			{
				UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::ReadUserFile] Unknown user %s. Cancelling transfer request for file %s"), *SharedUserId->ToString(), *FileName);
			}

			return EOS_PlayerDataStorage_EReadResult::EOS_RR_CancelRequest;
		});

	CallbackObj->SetNested2CallbackLambda([this, SharedUserId, FileName](const _tagEOS_PlayerDataStorage_FileTransferProgressCallbackInfo* Data)
		{
			UE_LOG_ONLINE_CLOUD(VeryVerbose, TEXT("[FOnlineUserCloudEOS::ReadUserFile] File transfer progress for file %s is %d bytes"), *FileName, Data->BytesTransferred);
			//TriggerOnReadUserFileProgressDelegates(Data->BytesTransferred, *SharedUserId, FileName); It doesn't exist, maybe we should add it
		});

	CallbackObj->CallbackLambda = [this, SharedUserId, FileName](const EOS_PlayerDataStorage_ReadFileCallbackInfo* Data)
	{
		bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;

		FUserCloudFileCollection* UserCloudFileCollection = FileSetsPerUser.Find(SharedUserId);
		if (UserCloudFileCollection != nullptr)
		{
			FEOSUserCloudFile* UserCloudFile = UserCloudFileCollection->Find(FileName);
			if (UserCloudFile != nullptr)
			{
				if (UserCloudFile->FileTransferRequest != nullptr)
				{
					EOS_PlayerDataStorageFileTransferRequest_Release(UserCloudFile->FileTransferRequest);
					UserCloudFile->FileTransferRequest = nullptr;
				}

				if (bWasSuccessful)
				{
					UserCloudFile->bIsLoaded = true;
					UserCloudFile->bInProgress = false;
					UE_LOG_ONLINE_CLOUD(Verbose, TEXT("[FOnlineUserCloudEOS::ReadUserFile] Read file %s with size %zu"), *UserCloudFile->Filename, UserCloudFile->ContentSize);
				}
				else
				{
					// If we fail to complete reading the file, discard it from the known files
					FileSetsPerUser.Find(SharedUserId)->Remove(FileName);

					UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::ReadUserFile] EOS_PlayerDataStorage_ReadFile was not successful for file %s. Finished with error %s"), *FileName, *LexToString(Data->ResultCode));
				}
			}
			else
			{
				bWasSuccessful = false;
				UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::ReadUserFile] Unknown transfer request for file %s"), *FileName);
			}
		}
		else
		{
			UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::ReadUserFile] Unknown user %s for file %s's transfer request"), *SharedUserId->ToString(), *FileName);
		}

		TriggerOnReadUserFileCompleteDelegates(bWasSuccessful, *SharedUserId, FileName);
	};

	const FUniqueNetIdEOS& UserEOSId = FUniqueNetIdEOS::Cast(UserId);

	EOS_PlayerDataStorage_ReadFileOptions ReadFileOptions = {};
	ReadFileOptions.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_PLAYERDATASTORAGE_READFILEOPTIONS_API_LATEST, 2);
	ReadFileOptions.LocalUserId = UserEOSId.GetProductUserId();
	ReadFileOptions.Filename = FileNameUtf8.Get();
	ReadFileOptions.ReadChunkLengthBytes = (uint32_t)ReadChunkSize;
	ReadFileOptions.ReadFileDataCallback = CallbackObj->GetNested1CallbackPtr();
	ReadFileOptions.FileTransferProgressCallback = CallbackObj->GetNested2CallbackPtr();

	EOS_HPlayerDataStorageFileTransferRequest FileTransferRequest = EOS_PlayerDataStorage_ReadFile(EOSSubsystem->PlayerDataStorageHandle, &ReadFileOptions, CallbackObj, CallbackObj->GetCallbackPtr());

	if (FileTransferRequest != nullptr)
	{
		FEOSUserCloudFile UserCloudFile;
		UserCloudFile.Filename = FileName;
		UserCloudFile.FileTransferRequest = FileTransferRequest;
		UserCloudFile.bInProgress = true;
		FileSetsPerUser.FindOrAdd(SharedUserId).FindOrAdd(FileName) = MoveTemp(UserCloudFile); // Replace the last file, or create a new entry, same with the user
	}
	else
	{
		EOSSubsystem->ExecuteNextTick([this, UserIdRef = UserId.AsShared(), FileName]()
			{
				UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::ReadUserFile] Failed to create a transfer request for user's %s file with name %s"), *UserIdRef->ToString(), *FileName);
				TriggerOnReadUserFileCompleteDelegates(false, *UserIdRef, FileName);
			});
	}

	return true;
}

bool FOnlineUserCloudEOS::WriteUserFile(const FUniqueNetId& UserId, const FString& FileName, TArray<uint8>& FileContents, bool bCompressBeforeUpload)
{
	FUniqueNetIdPtr UniqueNetId = EOSSubsystem->UserManager->GetUniquePlayerId(EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(UserId));
	if (!UniqueNetId.IsValid())
	{
		EOSSubsystem->ExecuteNextTick([this, UserIdRef = UserId.AsShared(), FileName]()
			{
				UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::WriteUserFile] Unable to write file. User %s is not a local user"), *UserIdRef->ToString());
				TriggerOnWriteUserFileCompleteDelegates(false, *UserIdRef, FileName);
			});

		return true;
	}

	FUniqueNetIdRef SharedUserId = UniqueNetId.ToSharedRef();

	// If this file is already being read or written, don't try to start it again.
	FUserCloudFileCollection* UserCloudFileCollection = FileSetsPerUser.Find(SharedUserId);
	if (UserCloudFileCollection != nullptr)
	{
		FEOSUserCloudFile* UserCloudFile = UserCloudFileCollection->Find(FileName);
		if (UserCloudFile != nullptr && UserCloudFile->bInProgress)
		{
			EOSSubsystem->ExecuteNextTick([this, UserIdRef = UserId.AsShared(), FileName]()
				{
					UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::WriteUserFile] File %s for user %s is already in the process of being read or written."), *FileName, *UserIdRef->ToString());
					TriggerOnWriteUserFileCompleteDelegates(false, *UserIdRef, FileName);
				});

			return true;
		}
	}

	// We prepare all the components for the Options object

	FTCHARToUTF8 FileNameUtf8(*FileName);

	// We'll use the same buffer size to write as we use to read
	int32 ReadChunkSize = UEOSSettings::GetSettings().TitleStorageReadChunkLength;
	if (ReadChunkSize <= 0)
	{
		UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::WriteUserFile] invalid size TitleStorageReadChunkLength %d"), ReadChunkSize);
		ReadChunkSize = 16 * 1024;
	}

	FWriteUserFileCompleteCallback* CallbackObj = new FWriteUserFileCompleteCallback(FOnlineUserCloudEOSWeakPtr(AsShared()));

	CallbackObj->SetNested1CallbackLambda([this, SharedUserId, FileName](const EOS_PlayerDataStorage_WriteFileDataCallbackInfo* Data, void* OutDataBuffer, uint32_t* OutDataWritten)
	{
		UE_LOG_ONLINE_CLOUD(Verbose, TEXT("[FOnlineUserCloudEOS::WriteUserFile] Writing file data for %s"), *FileName);

		FUserCloudFileCollection* UserCloudFileCollection = FileSetsPerUser.Find(SharedUserId);
		if (UserCloudFileCollection != nullptr)
		{
			FEOSUserCloudFile* UserCloudFile = UserCloudFileCollection->Find(FileName);
			if (UserCloudFile != nullptr)
			{
				check(UserCloudFile->bInProgress);

				size_t BytesToWrite = FMath::Min(Data->DataBufferLengthBytes, (uint32_t)(UserCloudFile->ContentSize - UserCloudFile->ContentIndex));

				if (BytesToWrite == 0)
				{
					return EOS_PlayerDataStorage_EWriteResult::EOS_WR_CompleteRequest;
				}
				if (UserCloudFile->ContentIndex + BytesToWrite <= UserCloudFile->ContentSize)
				{
					check(BytesToWrite > 0);
					FMemory::Memcpy(OutDataBuffer, static_cast<const void*>(&UserCloudFile->Contents[UserCloudFile->ContentIndex]), BytesToWrite);
					*OutDataWritten = static_cast<uint32_t>(BytesToWrite);

					UserCloudFile->ContentIndex += (size_t)*OutDataWritten;

					UE_LOG_ONLINE_CLOUD(Verbose, TEXT("[FOnlineUserCloudEOS::WriteUserFile] Wrote %zu bytes for file %s"), BytesToWrite, *FileName);

					return EOS_PlayerDataStorage_EWriteResult::EOS_WR_ContinueWriting;
				}
				else
				{
					UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::WriteUserFile] Read size exceeded specified file size for file %s"), *FileName);
					return EOS_PlayerDataStorage_EWriteResult::EOS_WR_FailRequest;
				}
			}
			else
			{
				UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::WriteUserFile] Unknown file %s. Cancelling transfer request"), *FileName);
			}
		}
		else
		{
			UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::WriteUserFile] Unknown user %s. Cancelling transfer request for file %s"), *SharedUserId->ToString(), *FileName);
		}

		return EOS_PlayerDataStorage_EWriteResult::EOS_WR_CancelRequest;
	});

	CallbackObj->SetNested2CallbackLambda([this, SharedUserId, FileName](const EOS_PlayerDataStorage_FileTransferProgressCallbackInfo* Data)
		{
			UE_LOG_ONLINE_CLOUD(VeryVerbose, TEXT("[FOnlineUserCloudEOS::WriteUserFile] File transfer progress for file %s is %d bytes"), *FileName, Data->BytesTransferred);

			TriggerOnWriteUserFileProgress64Delegates(Data->BytesTransferred, *SharedUserId, FileName);
		});

	CallbackObj->CallbackLambda = [this, SharedUserId, FileName](const EOS_PlayerDataStorage_WriteFileCallbackInfo* Data)
	{
		bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;

		FUserCloudFileCollection* UserCloudFileCollection = FileSetsPerUser.Find(SharedUserId);
		if (UserCloudFileCollection != nullptr)
		{
			FEOSUserCloudFile* UserCloudFile = UserCloudFileCollection->Find(FileName);
			if (UserCloudFile != nullptr)
			{
				if (UserCloudFile->FileTransferRequest != nullptr)
				{
					EOS_PlayerDataStorageFileTransferRequest_Release(UserCloudFile->FileTransferRequest);
					UserCloudFile->FileTransferRequest = nullptr;
				}

				if (bWasSuccessful)
				{
					UserCloudFile->bIsLoaded = true;
					UserCloudFile->bInProgress = false;
					UE_LOG_ONLINE_CLOUD(Verbose, TEXT("[FOnlineUserCloudEOS::WriteUserFile] Wrote file %s with size %zu"), *UserCloudFile->Filename, UserCloudFile->ContentSize);
				}
				else
				{
					// If we fail to complete writing the file, discard it from the known files
					FileSetsPerUser.Find(SharedUserId)->Remove(FileName);

					UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::WriteUserFile] EOS_PlayerDataStorage_WriteFile was not successful. Finished with error %s"), *LexToString(Data->ResultCode));
				}
			}
			else
			{
				bWasSuccessful = false;
				UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::WriteUserFile] Unknown transfer request for file %s"), *FileName);
			}
		}
		else
		{
			UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::WriteUserFile] Unknown user %s for file %s's transfer request"), *SharedUserId->ToString(), *FileName);
		}

		TriggerOnWriteUserFileCompleteDelegates(bWasSuccessful, *SharedUserId, FileName);
	};

	const FUniqueNetIdEOS& UserEOSId = FUniqueNetIdEOS::Cast(UserId);

	EOS_PlayerDataStorage_WriteFileOptions WriteFileOptions = {};
	WriteFileOptions.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_PLAYERDATASTORAGE_WRITEFILEOPTIONS_API_LATEST, 2);
	WriteFileOptions.LocalUserId = UserEOSId.GetProductUserId();
	WriteFileOptions.Filename = FileNameUtf8.Get();
	WriteFileOptions.ChunkLengthBytes = (uint32_t)ReadChunkSize;
	WriteFileOptions.WriteFileDataCallback = CallbackObj->GetNested1CallbackPtr();
	WriteFileOptions.FileTransferProgressCallback = CallbackObj->GetNested2CallbackPtr();

	EOS_HPlayerDataStorageFileTransferRequest FileTransferRequest = EOS_PlayerDataStorage_WriteFile(EOSSubsystem->PlayerDataStorageHandle, &WriteFileOptions, CallbackObj, CallbackObj->GetCallbackPtr());

	if (FileTransferRequest != nullptr)
	{
		FEOSUserCloudFile UserCloudFile;
		UserCloudFile.Filename = FileName;
		UserCloudFile.FileTransferRequest = FileTransferRequest;
		UserCloudFile.bInProgress = true;
		UserCloudFile.ContentSize = FileContents.Num();
		UserCloudFile.Contents = FileContents;
		FileSetsPerUser.FindOrAdd(SharedUserId).FindOrAdd(FileName) = MoveTemp(UserCloudFile); // Replace the last title file, or create a new entry, same with the user
	}
	else
	{
		EOSSubsystem->ExecuteNextTick([this, UserIdRef = UserId.AsShared(), FileName]()
			{
				UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::WriteUserFile] Failed to create a transfer request for user's %s file with name %s"), *UserIdRef->ToString(), *FileName);
				TriggerOnWriteUserFileCompleteDelegates(false, *UserIdRef, FileName);
			});		
	}

	return true;
}

void FOnlineUserCloudEOS::CancelWriteUserFile(const FUniqueNetId& UserId, const FString& FileName)
{
	FUniqueNetIdPtr UniqueNetId = EOSSubsystem->UserManager->GetUniquePlayerId(EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(UserId));
	if (!UniqueNetId.IsValid())
	{
		EOSSubsystem->ExecuteNextTick([this, UserIdRef = UserId.AsShared(), FileName]()
			{
				UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::CancelWriteUserFile] Unable to cancel file writing. User %s is not a local user"), *UserIdRef->ToString());
				TriggerOnWriteUserFileCanceledDelegates(false, *UserIdRef, FileName);
			});

		return;
	}

	bool bWasSuccessful = false;

	FUserCloudFileCollection* UserCloudFileCollection = FileSetsPerUser.Find(UniqueNetId.ToSharedRef());
	if (UserCloudFileCollection != nullptr)
	{
		FEOSUserCloudFile* UserCloudFile = UserCloudFileCollection->Find(FileName);
		if (UserCloudFile != nullptr)
		{
			if (UserCloudFile->bInProgress)
			{
				EOS_EResult Result = EOS_PlayerDataStorageFileTransferRequest_CancelRequest(UserCloudFile->FileTransferRequest);
				bWasSuccessful = Result == EOS_EResult::EOS_Success;
				if (bWasSuccessful)
				{
					UE_LOG_ONLINE_CLOUD(Verbose, TEXT("[FOnlineUserCloudEOS::CancelWriteUserFile] EOS_PlayerDataStorageFileTransferRequest_CancelRequest was successful. Cancelled write operation of file %s for user %s."), *FileName, *UserId.ToString());
				}
				else
				{
					// Result code will be EOS_NoChange if request had already completed (can't be canceled), and EOS_AlreadyPending if it's already been canceled before (this is a final state for a canceled request and won't change over time)
					UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::CancelWriteUserFile] EOS_PlayerDataStorageFileTransferRequest_CancelRequest was not successful. Finished with error %s"), *LexToString(Result));
				}
			}
			else
			{
				UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::CancelWriteUserFile] File %s is not in progress of reading or writing. Unable to cancel."), *FileName);
			}
		}
		else
		{
			UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::CancelWriteUserFile] Unknown file %s"), *FileName);
		}
	}
	else
	{
		UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::CancelWriteUserFile] Unknown user %s"), *UserId.ToString());
	}

	EOSSubsystem->ExecuteNextTick([this, bWasSuccessful, UserIdRef = UserId.AsShared(), FileName]()
		{
			TriggerOnWriteUserFileCanceledDelegates(bWasSuccessful, *UserIdRef, FileName);
		});
}

bool FOnlineUserCloudEOS::DeleteUserFile(const FUniqueNetId& UserId, const FString& FileName, bool bShouldCloudDelete, bool bShouldLocallyDelete)
{
	FUniqueNetIdPtr UniqueNetId = EOSSubsystem->UserManager->GetUniquePlayerId(EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(UserId));
	if (!UniqueNetId.IsValid())
	{
		EOSSubsystem->ExecuteNextTick([this, UserIdRef = UserId.AsShared(), FileName]()
			{
				UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::DeleteUserFile] Unable to delete user file. User %s is not a local user"), *UserIdRef->ToString());
				TriggerOnDeleteUserFileCompleteDelegates(false, *UserIdRef, FileName);
			});

		return true;
	}

	// Local deletion
	if (bShouldLocallyDelete)
	{
		bool bWasLocalDeletionSuccessful = false;

		FUserCloudFileCollection* UserCloudFileCollection = FileSetsPerUser.Find(UniqueNetId.ToSharedRef());
		if (UserCloudFileCollection != nullptr)
		{
			FEOSUserCloudFile* UserCloudFile = UserCloudFileCollection->Find(FileName);
			if (UserCloudFile != nullptr)
			{
				// If this file is being read or written, don't delete it.
				if (!UserCloudFile->bInProgress)
				{
					UserCloudFile->Unload();

					bWasLocalDeletionSuccessful = true;
				}
				else
				{
					UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::DeleteUserFile] Unable to delete. File %s for user %s is in the process of being read or written."), *FileName, *UserId.ToString());
				}
			}
			else
			{
				UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::DeleteUserFile] Unknown file (%s)"), *FileName);
			}
		}
		else
		{
			UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::DeleteUserFile] Unknown user (%s)"), *UserId.ToString());
		}

		if (!bShouldCloudDelete)
		{
			EOSSubsystem->ExecuteNextTick([this, bWasLocalDeletionSuccessful, UserIdRef = UserId.AsShared(), FileName]()
				{
					TriggerOnDeleteUserFileCompleteDelegates(bWasLocalDeletionSuccessful, *UserIdRef, FileName);
				});

			return true;
		}
	}

	// Cloud deletion
	if (bShouldCloudDelete)
	{
		FTCHARToUTF8 FileNameUtf8(*FileName);

		const FUniqueNetIdEOS& UserEOSId = FUniqueNetIdEOS::Cast(UserId);

		EOS_PlayerDataStorage_DeleteFileOptions Options = {};
		Options.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_PLAYERDATASTORAGE_DELETEFILEOPTIONS_API_LATEST, 1);
		Options.LocalUserId = UserEOSId.GetProductUserId();
		Options.Filename = FileNameUtf8.Get();

		FOnDeleteFileCallback* CallbackObj = new FOnDeleteFileCallback(FOnlineUserCloudEOSWeakPtr(AsShared()));
		CallbackObj->CallbackLambda = [this, UserIdRef = UserId.AsShared(), FileName](const EOS_PlayerDataStorage_DeleteFileCallbackInfo* Data)
		{
			bool bWasSuccessful = Data->ResultCode == EOS_EResult::EOS_Success;
			if (bWasSuccessful)
			{
				UE_LOG_ONLINE_CLOUD(Verbose, TEXT("[FOnlineUserCloudEOS::DeleteUserFile] EOS_PlayerDataStorage_DeleteFile was successful."));
			}
			else
			{
				// File deletion operations can fail if the user does not own the file
				UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::DeleteUserFile] EOS_PlayerDataStorage_DeleteFile was not successful for file %s. Finished with error %s"), *FileName, *LexToString(Data->ResultCode));
			}

			TriggerOnDeleteUserFileCompleteDelegates(bWasSuccessful, *UserIdRef, FileName);
		};

		EOS_PlayerDataStorage_DeleteFile(EOSSubsystem->PlayerDataStorageHandle, &Options, CallbackObj, CallbackObj->GetCallbackPtr());
	}

	return true;
}

bool FOnlineUserCloudEOS::RequestUsageInfo(const FUniqueNetId& UserId)
{
	UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::RequestUsageInfo] Not supported by API"));
	return false;
}

void FOnlineUserCloudEOS::DumpCloudState(const FUniqueNetId& UserId)
{
	FUniqueNetIdPtr UniqueNetId = EOSSubsystem->UserManager->GetUniquePlayerId(EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(UserId));
	if (!UniqueNetId.IsValid())
	{
		UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::DumpCloudState] Unable to dump cloud state. User %s is not a local user"), *UserId.ToString());
		return;
	}

	// Local deletion
	FUserCloudFileCollection* UserCloudFileCollection = FileSetsPerUser.Find(UniqueNetId.ToSharedRef());
	if (UserCloudFileCollection != nullptr)
	{
		UE_LOG_ONLINE_CLOUD(Log, TEXT("[FOnlineUserCloudEOS::DumpCloudState] User %s has %d files cached."), *UserId.ToString(), UserCloudFileCollection->Num());
	}
	else
	{
		UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::DumpCloudState] Unknown user (%s)"), *UserId.ToString());
	}
}

void FOnlineUserCloudEOS::DumpCloudFileState(const FUniqueNetId& UserId, const FString& FileName)
{
	FUniqueNetIdPtr UniqueNetId = EOSSubsystem->UserManager->GetUniquePlayerId(EOSSubsystem->UserManager->GetLocalUserNumFromUniqueNetId(UserId));
	if (!UniqueNetId.IsValid())
	{
		UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::DumpCloudFileState] Unable to dump cloud file state. User %s is not a local user"), *UserId.ToString());
		return;
	}

	FUserCloudFileCollection* UserCloudFileCollection = FileSetsPerUser.Find(UniqueNetId.ToSharedRef());
	if (UserCloudFileCollection != nullptr)
	{
		FEOSUserCloudFile* UserCloudFile = UserCloudFileCollection->Find(FileName);
		if (UserCloudFile != nullptr)
		{
			UE_LOG_ONLINE_CLOUD(Log, TEXT("[FOnlineUserCloudEOS::DumpCloudFileState] File %s state: ContentSize (%zu), IsLoaded (%d), InProgress (%d)"), *UserCloudFile->Filename, UserCloudFile->ContentSize, UserCloudFile->bIsLoaded, UserCloudFile->bInProgress);
		}
		else
		{
			UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::DumpCloudFileState] Unknown file (%s)"), *FileName);
		}
	}
	else
	{
		UE_LOG_ONLINE_CLOUD(Warning, TEXT("[FOnlineUserCloudEOS::DumpCloudFileState] Unknown user (%s)"), *UserId.ToString());
	}
}

void WriteRandomFile(TArray<uint8>& Buffer, int32 Size)
{
	Buffer.Empty(Size);
	Buffer.AddUninitialized(Size);
	for (int32 i = 0; i < Size; i++)
	{
		Buffer[i] = i % 255;
	}
}

bool FOnlineUserCloudEOS::HandleUserCloudExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;

	if (FParse::Command(&Cmd, TEXT("ENUMERATEUSERFILES")))
	{
		int UserIndex = FCString::Atoi(*FParse::Token(Cmd, false));

		EnumerateUserFiles(*EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(UserIndex));

		bWasHandled = true;
	}
	else if(FParse::Command(&Cmd, TEXT("GETUSERFILELIST")))
	{
		int UserIndex = FCString::Atoi(*FParse::Token(Cmd, false));

		TArray<FCloudFileHeader> UserFileList;
		GetUserFileList(*EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(UserIndex), UserFileList);

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("WRITEUSERFILE")))
	{
		int UserIndex = FCString::Atoi(*FParse::Token(Cmd, false));
		FString FileName = FParse::Token(Cmd, false);
		int32 FileSize = FCString::Atoi(*FParse::Token(Cmd, false));
		TArray<uint8> FileContents;
		WriteRandomFile(FileContents, FileSize);

		WriteUserFile(*EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(UserIndex), FileName, FileContents);

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("READUSERFILE")))
	{
		int UserIndex = FCString::Atoi(*FParse::Token(Cmd, false));
		FString FileName = FParse::Token(Cmd, false);

		ReadUserFile(*EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(UserIndex), FileName);

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("GETFILECONTENTS")))
	{
		int UserIndex = FCString::Atoi(*FParse::Token(Cmd, false));
		FString FileName = FParse::Token(Cmd, false);

		TArray<uint8> FileContents;
		GetFileContents(*EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(UserIndex), FileName, FileContents);

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("DELETEUSERFILE")))
	{
		int UserIndex = FCString::Atoi(*FParse::Token(Cmd, false));
		FString FileName = FParse::Token(Cmd, false);
		bool bShouldCloudDelete = (bool)FCString::Atoi(*FParse::Token(Cmd, false));
		bool bShouldLocallyDelete = (bool)FCString::Atoi(*FParse::Token(Cmd, false));

		DeleteUserFile(*EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(UserIndex), FileName, bShouldCloudDelete, bShouldLocallyDelete);

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("CLEARFILES")))
	{
		int UserIndex = FCString::Atoi(*FParse::Token(Cmd, false));

		ClearFiles(*EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(UserIndex));

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("CLEARFILE")))
	{
		int UserIndex = FCString::Atoi(*FParse::Token(Cmd, false));
		FString FileName = FParse::Token(Cmd, false);

		ClearFile(*EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(UserIndex), FileName);

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("CANCELWRITEUSERFILE")))
	{
		int UserIndex = FCString::Atoi(*FParse::Token(Cmd, false));
		FString FileName = FParse::Token(Cmd, false);

		CancelWriteUserFile(*EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(UserIndex), FileName);

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("DUMPCLOUDSTATE")))
	{
		int UserIndex = FCString::Atoi(*FParse::Token(Cmd, false));

		DumpCloudState(*EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(UserIndex));

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("DUMPCLOUDFILESTATE")))
	{
		int UserIndex = FCString::Atoi(*FParse::Token(Cmd, false));
		FString FileName = FParse::Token(Cmd, false);

		DumpCloudFileState(*EOSSubsystem->UserManager->GetLocalUniqueNetIdEOS(UserIndex), FileName);

		bWasHandled = true;
	}

	return bWasHandled;
}

#endif
