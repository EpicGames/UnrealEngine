// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Online/CoreOnline.h"
#include "Interfaces/OnlineUserCloudInterface.h"
#include "OnlineSubsystemEOSTypes.h"

class FOnlineSubsystemEOS;

#if WITH_EOS_SDK
	#include "eos_playerdatastorage_types.h"

struct FEOSUserCloudFile
{
	TArray<uint8> Contents;
	size_t ContentSize;
	size_t ContentIndex;
	bool bIsLoaded;
	bool bInProgress;
	FString Filename;
	EOS_HPlayerDataStorageFileTransferRequest FileTransferRequest;

	FEOSUserCloudFile() : ContentSize(0), ContentIndex(0), bIsLoaded(false), bInProgress(false)
	{
	}

	void Unload();
};

typedef TMap<FString, FEOSUserCloudFile> FUserCloudFileCollection;

class FOnlineUserCloudEOS
	: public IOnlineUserCloud, public TSharedFromThis<FOnlineUserCloudEOS, ESPMode::ThreadSafe>
{
public:
	FOnlineUserCloudEOS() = delete;
	virtual ~FOnlineUserCloudEOS() = default;

//~ IOnlineUserCloud
	virtual bool GetFileContents(const FUniqueNetId& UserId, const FString& FileName, TArray<uint8>& FileContents) override;
	virtual bool ClearFiles(const FUniqueNetId& UserId) override;
	virtual bool ClearFile(const FUniqueNetId& UserId, const FString& FileName) override;
	virtual void EnumerateUserFiles(const FUniqueNetId& UserId) override;
	virtual void GetUserFileList(const FUniqueNetId& UserId, TArray<FCloudFileHeader>& UserFiles) override;
	virtual bool ReadUserFile(const FUniqueNetId& UserId, const FString& FileName) override;
	virtual bool WriteUserFile(const FUniqueNetId& UserId, const FString& FileName, TArray<uint8>& FileContents, bool bCompressBeforeUpload = false) override;
	virtual void CancelWriteUserFile(const FUniqueNetId& UserId, const FString& FileName) override;
	virtual bool DeleteUserFile(const FUniqueNetId& UserId, const FString& FileName, bool bShouldCloudDelete, bool bShouldLocallyDelete) override;
	virtual bool RequestUsageInfo(const FUniqueNetId& UserId) override;
	virtual void DumpCloudState(const FUniqueNetId& UserId) override;
	virtual void DumpCloudFileState(const FUniqueNetId& UserId, const FString& FileName) override;
//~ IOnlineUserCloud

	FOnlineUserCloudEOS(FOnlineSubsystemEOS* InSubsystem)
		: EOSSubsystem(InSubsystem)
	{
	}

	bool HandleUserCloudExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);

protected:
	FOnlineSubsystemEOS* EOSSubsystem;

private:
	/** Results of the last file enumeration per user */
	TUniqueNetIdMap<TArray<FCloudFileHeader>> QueryFileSetsPerUser;

	/** The lists of available files per user, indexed by filename that have been or are loaded */
	TUniqueNetIdMap<FUserCloudFileCollection> FileSetsPerUser;
};

typedef TSharedPtr<FOnlineUserCloudEOS, ESPMode::ThreadSafe> FOnlineUserCloudEOSPtr;
typedef TWeakPtr<FOnlineUserCloudEOS, ESPMode::ThreadSafe> FOnlineUserCloudEOSWeakPtr;

#endif