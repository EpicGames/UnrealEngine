// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocketSubsystemEOSUtils_OnlineSubsystemEOS.h"

#include "OnlineSubsystemTypes.h"
#include "OnlineSubsystemEOS.h"
#include "UserManagerEOS.h"
#include "OnlineSessionEOS.h"

FSocketSubsystemEOSUtils_OnlineSubsystemEOS::FSocketSubsystemEOSUtils_OnlineSubsystemEOS(FOnlineSubsystemEOS& InSubsystemEOS)
	: SubsystemEOS(InSubsystemEOS)
{
}

FSocketSubsystemEOSUtils_OnlineSubsystemEOS::~FSocketSubsystemEOSUtils_OnlineSubsystemEOS()
{
}

#if WITH_EOS_P2P
EOS_ProductUserId FSocketSubsystemEOSUtils_OnlineSubsystemEOS::GetLocalUserId()
{
	EOS_ProductUserId Result = nullptr;

	check(SubsystemEOS.UserManager);
	Result = SubsystemEOS.UserManager->GetLocalProductUserId(SubsystemEOS.UserManager->GetDefaultLocalUser());

	return Result;
}
#endif // WITH_EOS_P2P

bool FSocketSubsystemEOSUtils_OnlineSubsystemEOS::IsLoggedIn()
{
	check(SubsystemEOS.UserManager);
	ELoginStatus::Type LoginStatus = SubsystemEOS.UserManager->GetLoginStatus(SubsystemEOS.UserManager->GetDefaultLocalUser());
	return LoginStatus == ELoginStatus::LoggedIn; 
}

FString FSocketSubsystemEOSUtils_OnlineSubsystemEOS::GetSessionId()
{
	FString Result;

	const IOnlineSessionPtr DefaultSessionInt = SubsystemEOS.GetSessionInterface();
	if (DefaultSessionInt.IsValid())
	{
		if (const FNamedOnlineSession* const NamedSession = DefaultSessionInt->GetNamedSession(NAME_GameSession))
		{
			Result = NamedSession->GetSessionIdStr();
		}
	}

	return Result;
}

FName FSocketSubsystemEOSUtils_OnlineSubsystemEOS::GetSubsystemInstanceName()
{
	return SubsystemEOS.GetInstanceName();
}
