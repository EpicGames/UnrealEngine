// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "SocketSubsystemEOS.h"

class FOnlineSubsystemEOS;

class FSocketSubsystemEOSUtils_OnlineSubsystemEOS : public ISocketSubsystemEOSUtils
{
public:
	FSocketSubsystemEOSUtils_OnlineSubsystemEOS(FOnlineSubsystemEOS& InSubsystemEOS);
	virtual ~FSocketSubsystemEOSUtils_OnlineSubsystemEOS() override;

#if WITH_EOS_P2P
	virtual EOS_ProductUserId GetLocalUserId() override;
#endif // WITH_EOS_P2P
	virtual FString GetSessionId() override;
	virtual FName GetSubsystemInstanceName() override;
	virtual bool IsLoggedIn() override;;

private:
	FOnlineSubsystemEOS& SubsystemEOS;
};