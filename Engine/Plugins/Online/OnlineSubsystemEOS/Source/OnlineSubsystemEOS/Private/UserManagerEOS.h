// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemEOSTypes.h"
#include "Interfaces/OnlineExternalUIInterface.h"
#include "Interfaces/OnlineFriendsInterface.h"
#include "Interfaces/OnlinePresenceInterface.h"

#if WITH_EOS_SDK
	#include "eos_auth_types.h"
	#include "eos_friends_types.h"
	#include "eos_connect_types.h"

#define INVALID_LOCAL_USER -1

template <typename ValueType>
using TLocalUserArray = TSparseArray<ValueType, TInlineSparseArrayAllocator<MAX_LOCAL_PLAYERS>>;

class FOnlineSubsystemEOS;
class IOnlineSubsystem;

typedef TSharedPtr<FOnlineUser> FOnlineUserPtr;
typedef TSharedRef<FOnlineUser> FOnlineUserRef;

/**
* Concrete online user class built from the reusable templates
*/
class FOnlineUserEOS :
	public TOnlineUserEOS<FOnlineUser, IAttributeAccessInterface>
{
public:
	FOnlineUserEOS(FUniqueNetIdEOSRef InUserId, const FOnlineSubsystemEOS& InSubsystem) :
		TOnlineUserEOS<FOnlineUser, IAttributeAccessInterface>(InUserId, InSubsystem)
	{
	}
	virtual ~FOnlineUserEOS() = default;
};

typedef TSharedPtr<FOnlineUserEOS> FOnlineUserEOSPtr;
typedef TSharedRef<FOnlineUserEOS> FOnlineUserEOSRef;

/**
 * Info associated with an user account generated by this online service
 */
class FUserOnlineAccountEOS : 
	public TUserOnlineAccountEOS<FUserOnlineAccount>
{
public:
	FUserOnlineAccountEOS(FUniqueNetIdEOSRef InUserId, FOnlineSubsystemEOS& InSubsystem)
		: TUserOnlineAccountEOS<FUserOnlineAccount>(InUserId, InSubsystem)
	{
	}
	virtual ~FUserOnlineAccountEOS() = default;
};

typedef TSharedPtr<FUserOnlineAccountEOS> FUserOnlineAccountEOSPtr;
typedef TSharedRef<FUserOnlineAccountEOS> FUserOnlineAccountEOSRef;

/**
 * Concrete friend class built from the reusable templates
 */
class FOnlineFriendEOS :
	public TOnlineFriendEOS<FOnlineFriend>
{
public:
	FOnlineFriendEOS(FUniqueNetIdEOSRef InUserId, const FOnlineSubsystemEOS& InSubsystem) :
		TOnlineFriendEOS<FOnlineFriend>(InUserId, InSubsystem)
	{
	}

	FOnlineFriendEOS(FUniqueNetIdEOSRef InUserId, const TMap<FString, FString>& InUserAttributes, const FOnlineSubsystemEOS& InSubsystem) :
		TOnlineFriendEOS<FOnlineFriend>(InUserId, InUserAttributes, InSubsystem)
	{
	}

	virtual ~FOnlineFriendEOS() = default;
};

typedef TSharedPtr<FOnlineFriendEOS> FOnlineFriendEOSPtr;
typedef TSharedRef<FOnlineFriendEOS> FOnlineFriendEOSRef;

template<class ListClass, class ListClassReturnType>
class TOnlinePlayerList
{
	/** The user that owns this list */
	int32 LocalUserNum;
	/** The net id that owns this list */
	FUniqueNetIdEOSRef OwningNetId;
	/** The array of list class entries */
	TArray<ListClass> ListEntries;
	/** Indexed by unique id for fast look up */
	TUniqueNetIdMap<ListClass> UniqueIdToListEntryMap;

public:
	TOnlinePlayerList(int32 InLocalUserNum, FUniqueNetIdEOSRef InOwningNetId)
		: LocalUserNum(InLocalUserNum)
		, OwningNetId(InOwningNetId)
	{
	}

	const TArray<ListClass>& GetList()
	{
		return ListEntries;
	}

	void Add(const FUniqueNetIdRef& InNetId, ListClass InListEntry)
	{
		ListEntries.Add(InListEntry);
		UniqueIdToListEntryMap.Add(InNetId, InListEntry);
	}

	void Remove(const FUniqueNetIdRef& InNetId, ListClass InListEntry)
	{
		UniqueIdToListEntryMap.Remove(InNetId);
		ListEntries.Remove(InListEntry);
	}

	void Empty(int32 Slack = 0)
	{
		ListEntries.Empty(Slack);
		UniqueIdToListEntryMap.Empty(Slack);
	}

	ListClassReturnType GetByIndex(int32 Index)
	{
		if (ListEntries.IsValidIndex(Index))
		{
			return ListEntries[Index];
		}
		return ListClassReturnType();
	}

	ListClassReturnType GetByNetId(const FUniqueNetIdRef& NetId)
	{
		const ListClass* Found = UniqueIdToListEntryMap.Find(NetId);
		if (Found != nullptr)
		{
			return *Found;
		}
		return ListClassReturnType();
	}
};

class FFriendsListEOS :
	public TOnlinePlayerList<FOnlineFriendEOSRef, FOnlineFriendEOSPtr>
{
public:
	FFriendsListEOS(int32 InLocalUserNum, FUniqueNetIdEOSRef InOwningNetId)
		: TOnlinePlayerList<FOnlineFriendEOSRef, FOnlineFriendEOSPtr>(InLocalUserNum, InOwningNetId)
	{
	}
	virtual ~FFriendsListEOS() = default;
};
typedef TSharedRef<FFriendsListEOS> FFriendsListEOSRef;
typedef TSharedPtr<FFriendsListEOS> FFriendsListEOSPtr;

struct FNotificationIdCallbackPair
{
	EOS_NotificationId NotificationId;
	FCallbackBase* Callback;

	FNotificationIdCallbackPair()
		: NotificationId(EOS_INVALID_NOTIFICATIONID)
		, Callback(nullptr)
	{
	}

	virtual ~FNotificationIdCallbackPair()
	{
		delete Callback;
	}
};

/** Cache for the info passed on to ReadFriendsList, kept while user info queries complete */
struct ReadUserListInfo
{
	const int32 LocalUserNum;
	const FString ListName;
	const FOnReadFriendsListComplete Delegate;

	ReadUserListInfo(int32 InLocalUserNum, const FString& InListName, FOnReadFriendsListComplete InDelegate)
		: LocalUserNum(InLocalUserNum), ListName(InListName), Delegate(MoveTemp(InDelegate))
	{
	}

	void ExecuteDelegateIfBound(bool bWasSuccessful, const FString& ErrorStr) const
	{
		Delegate.ExecuteIfBound(LocalUserNum, bWasSuccessful, ListName, ErrorStr);
	};
};

/** Created when a new local user starts a login process, keeps the state for the login session and ongoing operations, and gets destroyed when login fails or the user logs out */
struct FLocalUserEOS
{
	/** UniqueNetId for the local user, set after successful authentication */
	FUniqueNetIdEOSPtr UniqueNetId;

	/** Online Account information for the local user */
	FUserOnlineAccountEOSPtr UserOnlineAccount;

	/** Friends information for the local user. Populated by ReadFriendsList */
	FFriendsListEOSPtr FriendsList;

	/** Epic ids for users with ongoing queries */
	TArray<EOS_EpicAccountId> OngoingQueryUserInfoAccounts;

	struct FReadUserInfoResults
	{
		/** UniqueNetIds gathered as a result of successful queries */
		TArray<FUniqueNetIdEOSRef> ProcessedIds;

		/** Whether all queries for this given operation were successful */
		bool bAllWasSuccessful = true;

		/** Combined error str for all operations */
		FString AllErrorStr;

		void Reset()
		{
			ProcessedIds.Empty();
			bAllWasSuccessful = true;
			AllErrorStr.Empty();
		}
	} OngoingQueryUserInfoResults;

	/** External ids still in process of resolution for local user */
	TArray<FString> OngoingPlayerQueryExternalMappings;

	/** This keeps track of all delegates passed on to calls to ReadFriendsList, so they will all be executed once the process finishes */
	TArray<ReadUserListInfo> CachedReadUserListInfo;

	/** Last Login Credentials used for a login attempt */
	TSharedPtr<FOnlineAccountCredentials> LastLoginCredentials_Legacy;

	/** If set, Connect Login will be re-attempted with it. If this attempt fails, a new token will need to be provided */
	TOptional<FString> UserProvidedConnectAuthToken;

	/** Track if the player is in the process of logging out to avoid calling EOS SDK logout APIs multiple times */
	bool bIsLoggingOut = false;
};

/**
 * EOS service implementation of the online identity, friends, and user info interfaces
 *
 * Centrally managed since they share common data
 */
class FUserManagerEOS
	: public IOnlineIdentity
	, public IOnlineExternalUI
	, public IOnlineFriends
	, public IOnlinePresence
	, public IOnlineUser
	, public TSharedFromThis<FUserManagerEOS, ESPMode::ThreadSafe>
{
public:
	/**
	 * Constructor
	 *
	 * @param InSubsystem online subsystem being used
	 */
	FUserManagerEOS(FOnlineSubsystemEOS* InSubsystem);

	/**
	 * Destructor
	 */
	virtual ~FUserManagerEOS();

	void Init();
	void Shutdown();

// IOnlineIdentity Interface
	virtual bool Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials) override;
	virtual bool Logout(int32 LocalUserNum) override;
	virtual bool AutoLogin(int32 LocalUserNum) override;
	virtual TSharedPtr<FUserOnlineAccount> GetUserAccount(const FUniqueNetId& UserId) const override;
	virtual TArray<TSharedPtr<FUserOnlineAccount>> GetAllUserAccounts() const override;
	virtual FUniqueNetIdPtr GetUniquePlayerId(int32 LocalUserNum) const override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(uint8* Bytes, int32 Size) override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(const FString& Str) override;
	virtual ELoginStatus::Type GetLoginStatus(int32 LocalUserNum) const override;
	virtual ELoginStatus::Type GetLoginStatus(const FUniqueNetId& UserId) const override;
	virtual FString GetPlayerNickname(int32 LocalUserNum) const override;
	virtual FString GetPlayerNickname(const FUniqueNetId& UserId) const override;
	virtual FString GetAuthToken(int32 LocalUserNum) const override;
	virtual void GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI = EShowPrivilegeResolveUI::Default) override;
	virtual FString GetAuthType() const override;
	virtual void RevokeAuthToken(const FUniqueNetId& LocalUserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate) override;
	virtual FPlatformUserId GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& UniqueNetId) const override;
	virtual void GetLinkedAccountAuthToken(int32 LocalUserNum, const FString& TokenType, const FOnGetLinkedAccountAuthTokenCompleteDelegate& Delegate) const override;
// ~IOnlineIdentity Interface

	ELoginStatus::Type GetLoginStatus(const FUniqueNetIdEOS& UserId) const;

// IOnlineExternalUI Interface
	virtual bool ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate = FOnLoginUIClosedDelegate()) override;
	virtual bool ShowAccountCreationUI(const int ControllerIndex, const FOnAccountCreationUIClosedDelegate& Delegate = FOnAccountCreationUIClosedDelegate()) override;
	virtual bool ShowFriendsUI(int32 LocalUserNum) override;
	virtual bool ShowInviteUI(int32 LocalUserNum, FName SessionName = NAME_GameSession) override;
	virtual bool ShowAchievementsUI(int32 LocalUserNum) override;
	virtual bool ShowLeaderboardUI(const FString& LeaderboardName) override;
	virtual bool ShowWebURL(const FString& Url, const FShowWebUrlParams& ShowParams, const FOnShowWebUrlClosedDelegate& Delegate = FOnShowWebUrlClosedDelegate()) override;
	virtual bool CloseWebURL() override;
	virtual bool ShowProfileUI(const FUniqueNetId& Requestor, const FUniqueNetId& Requestee, const FOnProfileUIClosedDelegate& Delegate = FOnProfileUIClosedDelegate()) override;
	virtual bool ShowAccountUpgradeUI(const FUniqueNetId& UniqueId) override;
	virtual bool ShowStoreUI(int32 LocalUserNum, const FShowStoreParams& ShowParams, const FOnShowStoreUIClosedDelegate& Delegate = FOnShowStoreUIClosedDelegate()) override;
	virtual bool ShowSendMessageUI(int32 LocalUserNum, const FShowSendMessageParams& ShowParams, const FOnShowSendMessageUIClosedDelegate& Delegate = FOnShowSendMessageUIClosedDelegate()) override;
// ~IOnlineExternalUI Interface

// IOnlineFriends Interface
	virtual bool ReadFriendsList(int32 LocalUserNum, const FString& ListName, const FOnReadFriendsListComplete& Delegate = FOnReadFriendsListComplete()) override;
	virtual bool DeleteFriendsList(int32 LocalUserNum, const FString& ListName, const FOnDeleteFriendsListComplete& Delegate = FOnDeleteFriendsListComplete()) override;
	virtual bool SendInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnSendInviteComplete& Delegate = FOnSendInviteComplete()) override;
	virtual bool AcceptInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnAcceptInviteComplete& Delegate = FOnAcceptInviteComplete()) override;
	virtual bool RejectInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual bool DeleteFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual bool GetFriendsList(int32 LocalUserNum, const FString& ListName, TArray< TSharedRef<FOnlineFriend> >& OutFriends) override;
	virtual TSharedPtr<FOnlineFriend> GetFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual bool IsFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual bool QueryRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace) override;
	virtual bool GetRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace, TArray< TSharedRef<FOnlineRecentPlayer> >& OutRecentPlayers) override;
	virtual bool BlockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId) override;
	virtual bool UnblockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId) override;
	virtual bool QueryBlockedPlayers(const FUniqueNetId& UserId) override;
	virtual bool GetBlockedPlayers(const FUniqueNetId& UserId, TArray< TSharedRef<FOnlineBlockedPlayer> >& OutBlockedPlayers) override;
	virtual void DumpBlockedPlayers() const override;
	virtual void SetFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FString& Alias, const FOnSetFriendAliasComplete& Delegate = FOnSetFriendAliasComplete()) override;
	virtual void DeleteFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnDeleteFriendAliasComplete& Delegate = FOnDeleteFriendAliasComplete()) override;
	virtual void DumpRecentPlayers() const override;
// ~IOnlineFriends Interface

	bool HandleFriendsExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);

// IOnlinePresence Interface
	virtual void SetPresence(const FUniqueNetId& User, const FOnlineUserPresenceStatus& Status, const FOnPresenceTaskCompleteDelegate& Delegate = FOnPresenceTaskCompleteDelegate()) override;
	virtual void QueryPresence(const FUniqueNetId& User, const FOnPresenceTaskCompleteDelegate& Delegate = FOnPresenceTaskCompleteDelegate()) override;
	virtual EOnlineCachedResult::Type GetCachedPresence(const FUniqueNetId& User, TSharedPtr<FOnlineUserPresence>& OutPresence) override;
	virtual EOnlineCachedResult::Type GetCachedPresenceForApp(const FUniqueNetId& LocalUserId, const FUniqueNetId& User, const FString& AppId, TSharedPtr<FOnlineUserPresence>& OutPresence) override;
// ~IOnlinePresence Interface

// IOnlineUser Interface
	virtual bool QueryUserInfo(int32 LocalUserNum, const TArray<FUniqueNetIdRef>& UserIds) override;
	virtual bool GetAllUserInfo(int32 LocalUserNum, TArray<TSharedRef<class FOnlineUser>>& OutUsers) override;
	virtual TSharedPtr<FOnlineUser> GetUserInfo(int32 LocalUserNum, const class FUniqueNetId& UserId) override;
	virtual bool QueryUserIdMapping(const FUniqueNetId& UserId, const FString& DisplayNameOrEmail, const FOnQueryUserMappingComplete& Delegate = FOnQueryUserMappingComplete()) override;
	virtual bool QueryExternalIdMappings(const FUniqueNetId& UserId, const FExternalIdQueryOptions& QueryOptions, const TArray<FString>& ExternalIds, const FOnQueryExternalIdMappingsComplete& Delegate = FOnQueryExternalIdMappingsComplete()) override;
	virtual void GetExternalIdMappings(const FExternalIdQueryOptions& QueryOptions, const TArray<FString>& ExternalIds, TArray<FUniqueNetIdPtr>& OutIds) override;
	virtual FUniqueNetIdPtr GetExternalIdMapping(const FExternalIdQueryOptions& QueryOptions, const FString& ExternalId) override;
// ~IOnlineUser Interface

	const FUniqueNetIdEOSPtr GetLocalUniqueNetIdEOS(int32 LocalUserNum) const;
	EOS_EpicAccountId GetLocalEpicAccountId(int32 LocalUserNum) const;
	EOS_ProductUserId GetLocalProductUserId(int32 LocalUserNum) const;

	int32 GetLocalUserNumFromUniqueNetId(const FUniqueNetId& NetId) const;
	int32 GetLocalUserNumFromEpicAccountId(const EOS_EpicAccountId& EpicAccountId) const;
	int32 GetLocalUserNumFromProductUserId(const EOS_ProductUserId& ProductUserId) const;

	const FUniqueNetIdEOSPtr GetLocalUniqueNetIdEOS(const EOS_EpicAccountId& EpicAccountId) const;
	const FUniqueNetIdEOSPtr GetLocalUniqueNetIdEOS(const EOS_ProductUserId& ProductUserId) const;

	bool IsLocalUser(const FUniqueNetId& NetId) const;
	bool IsLocalUser(const EOS_EpicAccountId& EpicAccountId) const;
	bool IsLocalUser(const EOS_ProductUserId& ProductUserId) const;

	using FResolveUniqueNetIdsCallback = TFunction<void(TMap<EOS_ProductUserId, FUniqueNetIdEOSRef> ResolvedUniqueNetIds, const FOnlineError& Error)>;
	using FResolveEpicAccountIdsCallback = TFunction<void(bool bWasSuccessful, TMap<EOS_EpicAccountId, FUniqueNetIdEOSRef> ResolvedUniqueNetIds, const FOnlineError& Error)>;
	using FResolveUniqueNetIdCallback = TFunction<void(FUniqueNetIdEOSRef ResolvedUniqueNetId, const FOnlineError& Error)>;

	bool GetEpicAccountIdFromProductUserId(int32 LocalUserNum, const EOS_ProductUserId& ProductUserId, EOS_EpicAccountId& OutEpicAccountId, FOnlineError& Error) const;
	void ResolveUniqueNetId(int32 LocalUserNum, const EOS_ProductUserId& ProductUserId, const FResolveUniqueNetIdCallback& Callback) const;
	void ResolveUniqueNetIds(int32 LocalUserNum, const TArray<EOS_ProductUserId>& ProductUserIds, const FResolveUniqueNetIdsCallback& Callback) const;
	void ResolveUniqueNetId(int32 LocalUserNum, const EOS_EpicAccountId& EpicAccountId, const FResolveUniqueNetIdCallback& Callback);
	void ResolveUniqueNetIds(int32 LocalUserNum, const TArray<EOS_EpicAccountId>& EpicAccountId, const FResolveEpicAccountIdsCallback& Callback);

	FOnlineUserPtr GetLocalOnlineUser(int32 LocalUserNum) const;
	FOnlineUserPtr GetOnlineUser(EOS_ProductUserId UserId) const;
	FOnlineUserPtr GetOnlineUser(EOS_EpicAccountId AccountId) const;

	/**
	 * Should use the initialization constructor instead
	 */
	FUserManagerEOS() = delete;

	bool ConnectLoginEAS(int32 LocalUserNum, EOS_EpicAccountId AccountId);
	void LoginViaPersistentAuth(int32 LocalUserNum);
	void LoginViaExternalAuth(int32 LocalUserNum);
	void CreateConnectedLogin(int32 LocalUserNum, EOS_EpicAccountId AccountId, EOS_ContinuanceToken Token);
	void LinkEAS(int32 LocalUserNum, EOS_ContinuanceToken Token);
	void AutoRefreshConnectLogin(const EOS_Connect_AuthExpirationCallbackInfo* Data);
	bool ConnectLoginNoEAS(int32 LocalUserNum);

	void FullLoginCallback(int32 LocalUserNum, EOS_EpicAccountId AccountId, EOS_ProductUserId UserId);
	void FriendStatusChanged(const EOS_Friends_OnFriendsUpdateInfo* Data);
	void FriendStatusChangedImpl(EOS_EpicAccountId LocalUserId, EOS_EpicAccountId TargetUserId, EOS_EFriendsStatus PreviousStatus, EOS_EFriendsStatus CurrentStatus);
	void AuthLoginStatusChanged(const EOS_Auth_LoginStatusChangedCallbackInfo* Data);
	void ConnectLoginStatusChanged(const EOS_Connect_LoginStatusChangedCallbackInfo* Data);
	void PresenceChanged(const EOS_Presence_PresenceChangedCallbackInfo* Data);

	int32 GetDefaultLocalUser() const { return DefaultLocalUser; }

	FString GetBestDisplayName(EOS_EpicAccountId TargetUserId, const FStringView& RequestedPlatform) const;

private:
	bool IsLocalUserValid(int32 LocalUserNum) const ;
	FLocalUserEOS& GetLocalUserChecked(int32 LocalUserNum);

	void CallEOSAuthLogin(int32 LocalUserNum, const FOnlineAccountCredentials& Credentials, bool bIsAutoLogin);
	void CopyAndSaveEpicAuthToken(int32 LocalUserNum, const EOS_EpicAccountId& EpicAccountId);
	void OnEOSAuthLoginComplete(int32 LocalUserNum, const EOS_ELoginCredentialType LoginCredentialType, const bool bIsAutoLogin, const EOS_Auth_LoginCallbackInfo* Data);
	void CallEOSConnectLogin(int32 LocalUserNum, const FOnlineAccountCredentials& Credentials);

	void CallEOSAuthLogout(int32 LocalUserNum, FUniqueNetIdEOSPtr UserId);
	void CallEOSConnectLogout(int32 LocalUserNum, FUniqueNetIdEOSPtr UserId);
	
	FLocalUserEOS& AddLocalUser(int32 LocalUserNum);
	FLocalUserEOS& UpdateLocalUser(int32 LocalUserNum, EOS_EpicAccountId EpicAccountId, EOS_ProductUserId UserId);
	void RemoveLocalUser(int32 LocalUserNum);

	typedef TFunction<void(bool bWasSuccessful, FUniqueNetIdEOSRef RemotePlayerNetId, const FString& ErrorStr)> FRemoteUserProcessedCallback;
	void AddRemotePlayer(int32 LocalUserNum, EOS_EpicAccountId EpicAccountId, const FRemoteUserProcessedCallback& Callback);
	typedef TFunction<void(bool bWasSuccessful, TArray<FUniqueNetIdEOSRef> RemotePlayerNetIds, const FString& ErrorStr)> FRemoteUsersProcessedCallback;
	void AddRemotePlayers(int32 LocalUserNum, TArray<EOS_EpicAccountId> EpicAccountIds, const FRemoteUsersProcessedCallback& Callback);
	void ReadUserInfo(int32 LocalUserNum, EOS_EpicAccountId EpicAccountId, const FRemoteUserProcessedCallback& Callback);

	// This method does not call AddRemotePlayer, only registers an already existing remote player as a friend for a local user
	FOnlineFriendEOSRef AddFriend(int32 LocalUserNum, const FUniqueNetIdEOS& FriendNetId);

	void UpdateUserInfo(IAttributeAccessInterfaceRef AttriubteAccessRef, EOS_EpicAccountId LocalId, EOS_EpicAccountId TargetId);
	bool IsFriendQueryUserInfoOngoing(int32 LocalUserNum);
	void ProcessReadFriendsListComplete(int32 LocalUserNum, bool bWasSuccessful, const FString& ErrorStr);

	void UpdatePresence(int32 LocalUserNum, EOS_EpicAccountId AccountId);
	void UpdateFriendPresence(const FUniqueNetIdEOSRef& FriendId, FOnlineUserPresenceRef Presence);

	void GetPlatformAuthToken(int32 LocalUserNum, const FOnGetLinkedAccountAuthTokenCompleteDelegate& Delegate) const;
	FString GetPlatformDisplayName(int32 LocalUserNum) const;

	/** Get the file name for the Epic Auth Token file used in Persistent Auth */
	FString GetEOSAuthTokenFilename();

	/** Cached pointer to owning subsystem */
	FOnlineSubsystemEOS* EOSSubsystem;

	/** Default local user when no local user is specified */
	int32 DefaultLocalUser;

	/** Notification state for SDK events */
	EOS_NotificationId AuthLoginStatusChangedNotificationId = EOS_INVALID_NOTIFICATIONID;
	FCallbackBase* AuthLoginStatusChangedNotificationCallback = nullptr;
	EOS_NotificationId ConnectLoginStatusChangedNotificationId = EOS_INVALID_NOTIFICATIONID;
	FCallbackBase* ConnectLoginStatusChangedNotificationCallback = nullptr;
	EOS_NotificationId ConnectLoginExpiredNotificationId = EOS_INVALID_NOTIFICATIONID;
	FCallbackBase* ConnectLoginExpiredNotificationCallback = nullptr;
	EOS_NotificationId FriendsNotificationId = EOS_INVALID_NOTIFICATIONID;
	FCallbackBase* FriendsNotificationCallback = nullptr;
	EOS_NotificationId PresenceNotificationId = EOS_INVALID_NOTIFICATIONID;
	FCallbackBase* PresenceNotificationCallback = nullptr;
	EOS_NotificationId DisplaySettingsUpdatedId = EOS_INVALID_NOTIFICATIONID;
	FCallbackBase* DisplaySettingsUpdatedCallback = nullptr;
	bool bLastReportedExclusiveInput = false;

	/** Information related to ongoing operations and state for local users */
	TLocalUserArray<FLocalUserEOS> LocalUsers;

	// Online user info

	/** Ids mapped to attribute access information for any user */
	TUniqueNetIdMap<FOnlineUserEOSRef> UniqueNetIdToUserRefMap;

	/** Ids mapped to user presence information for any user */
	TUniqueNetIdMap<FOnlineUserPresenceRef> UniqueNetIdToOnlineUserPresenceMap;
};

typedef TSharedPtr<FUserManagerEOS, ESPMode::ThreadSafe> FUserManagerEOSPtr;
typedef TWeakPtr<FUserManagerEOS, ESPMode::ThreadSafe> FUserManagerEOSWeakPtr;
typedef TWeakPtr<const FUserManagerEOS, ESPMode::ThreadSafe> FUserManagerEOSConstWeakPtr;

#endif
