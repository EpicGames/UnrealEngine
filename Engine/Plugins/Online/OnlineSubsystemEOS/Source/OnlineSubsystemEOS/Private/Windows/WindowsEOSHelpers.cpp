// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsEOSHelpers.h"

#if WITH_EOS_SDK

#include "EOSShared.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#if WITH_EOS_RTC
#include "Windows/eos_Windows.h"
#endif

IEOSPlatformHandlePtr FWindowsEOSHelpers::CreatePlatform(EOS_Platform_Options& PlatformOptions)
{
#if WITH_EOS_RTC
	const FTCHARToUTF8 Utf8XAudioPath(*IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*FPaths::Combine(FPaths::EngineDir(), TEXT("Binaries/ThirdParty/Windows/XAudio2_9/x64/xaudio2_9redist.dll"))));

	EOS_Windows_RTCOptions WindowsRtcOptions = { 0 };
	WindowsRtcOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_WINDOWS_RTCOPTIONS_API_LATEST, 1);
	WindowsRtcOptions.XAudio29DllPath = Utf8XAudioPath.Get();

	EOS_Platform_RTCOptions* RTCOptions = const_cast<EOS_Platform_RTCOptions*>(PlatformOptions.RTCOptions);
	if (RTCOptions)
	{
		RTCOptions->PlatformSpecificOptions = &WindowsRtcOptions;
	}
#endif // WITH_EOS_RTC

	return FEOSHelpers::CreatePlatform(PlatformOptions);
}

#endif // WITH_EOS_SDK