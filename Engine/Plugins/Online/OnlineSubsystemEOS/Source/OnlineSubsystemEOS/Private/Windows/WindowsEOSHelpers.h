// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EOS_SDK

#include "EOSHelpers.h"

class FWindowsEOSHelpers : public FEOSHelpers
{
public:
	virtual ~FWindowsEOSHelpers() = default;

	virtual IEOSPlatformHandlePtr CreatePlatform(EOS_Platform_Options& PlatformOptions) override;
};

using FPlatformEOSHelpers = FWindowsEOSHelpers;

#endif // WITH_EOS_SDK