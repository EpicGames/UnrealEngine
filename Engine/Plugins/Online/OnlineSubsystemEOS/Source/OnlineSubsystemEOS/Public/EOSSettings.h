// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Engine/RuntimeOptionsBase.h"
#include "Engine/DataAsset.h"
#include "EOSShared.h"
#include "EOSSettings.generated.h"

#define UE_API ONLINESUBSYSTEMEOS_API

/** Native version of the UObject based config data */
struct FEOSArtifactSettings
{
	FString ArtifactName;
	FString ClientId;
	FString ClientSecret;
	FString ProductId;
	FString SandboxId;
	FString DeploymentId;
	FString EncryptionKey;
};

UCLASS(Deprecated)
class UDEPRECATED_EOSArtifactSettings :
	public UDataAsset
{
	GENERATED_BODY()

public:
	UDEPRECATED_EOSArtifactSettings()
	{
	}
};

USTRUCT(BlueprintType)
struct FArtifactSettings
{
	GENERATED_BODY()

public:
	/** This needs to match what the launcher passes in the -epicapp command line arg */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="EOS Settings")
	FString ArtifactName;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Artifact Settings")
	FString ClientId;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Artifact Settings")
	FString ClientSecret;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Artifact Settings")
	FString ProductId;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Artifact Settings")
	FString SandboxId;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Artifact Settings")
	FString DeploymentId;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Artifact Settings")
	// Config key renamed to ClientEncryptionKey as EncryptionKey gets removed from packaged builds due to IniKeyDenylist=EncryptionKey entry in BaseGame.ini.
	FString ClientEncryptionKey;

	FEOSArtifactSettings ToNative() const;
};

/** Native version of the UObject based config data */
struct FEOSSettings
{
	FEOSSettings();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FEOSSettings(const FEOSSettings& Other) = default;
	FEOSSettings(FEOSSettings&& Other) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FString CacheDir;
	FString DefaultArtifactName;
	FString SteamTokenType;
	FString NintendoTokenType;
	FString PlatformConfigName;
	EOS_ERTCBackgroundMode RTCBackgroundMode;
	int32 TickBudgetInMilliseconds;
	int32 TitleStorageReadChunkLength;
	bool bEnableOverlay;
	bool bEnableSocialOverlay;
	bool bEnableEditorOverlay;
	bool bPreferPersistentAuth;
	bool bUseEAS;
	bool bUseEOSConnect;
	bool bUseEOSRTC;
	bool bUseNamedPlatformConfig;
	bool bUseNewEcomFlow;
	UE_DEPRECATED(5.7, "bUseNewLoginFlow is deprecated, the legacy login flow has been removed from UserManagerEOS.")
	bool bUseNewLoginFlow;
	TArray<FEOSArtifactSettings> Artifacts;
	TArray<FString> TitleStorageTags;
	TArray<FString> AuthScopeFlags;
};

UCLASS(MinimalAPI, Config=Engine, DefaultConfig)
class UEOSSettings :
	public URuntimeOptionsBase
{
	GENERATED_BODY()

public:
	/**
	 * The directory any PDS/TDS files are cached into. This is per artifact e.g.:
	 *
	 * <UserDir>/<ArtifactId>/<CacheDir>
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Settings")
	FString CacheDir = TEXT("CacheDir");

	/** Used when launched from a store other than EGS or when the specified artifact name was not present */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Settings")
	FString DefaultArtifactName;

	/** When bUseNamedPlatformConfig is true, specifies the platform config name to use */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Settings")
	FString PlatformConfigName;

	/** The preferred background mode to be used by RTC services */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Settings")
	FString RTCBackgroundMode;

	/** Used to throttle how much time EOS ticking can take */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Settings")
	int32 TickBudgetInMilliseconds = 0;

	/** Set to true to enable the overlay (ecom features) */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Settings")
	bool bEnableOverlay = false;

	/** Set to true to enable the social overlay (friends, invites, etc.) */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Settings")
	bool bEnableSocialOverlay = false;

	/** Set to true to enable the overlay when running in the editor */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "EOS Settings")
	bool bEnableEditorOverlay = false;

	/** Set to true to prefer persistent auth over external authentication during Login */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Settings")
	bool bPreferPersistentAuth = false;

	/** Tag combinations for paged queries in title file enumerations, separate tags within groups using `+` */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Settings")
	TArray<FString> TitleStorageTags;

	/** Chunk size used when reading a title file */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Settings")
	int32 TitleStorageReadChunkLength = 0;

	/** Per artifact SDK settings. A game might have a FooStaging, FooQA, and public Foo artifact */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Settings")
	TArray<FArtifactSettings> Artifacts;

	/** Auth scopes to request during login */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "EOS Settings")
	TArray<FString> AuthScopeFlags;

	/** Set to true to login to EOS_Auth (required to use Epic Account Services) */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Settings",  DisplayName="Login to EOS Auth, and use Epic Account Services")
	bool bUseEAS = false;

	/** Set to true to login to EOS_Connect (required to use Epic Game Services) */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Settings", DisplayName="Login to EOS Connect, and use EOS Game Services")
	bool bUseEOSConnect = false;

	/** Whether real-time chat is initialized when creating the EOS platform */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Settings", DisplayName="Enable Real-Time chat")
	bool bUseEOSRTC = true;

	/**
	 * Set to true to use the IEOSSDKManager "named platform config" mechanism, rather than
	 * loading a config from OSSEOS ini. Specify the platform config name to use with the
	 * "PlatformConfigName" setting.
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Settings")
	bool bUseNamedPlatformConfig = false;

	/** New Epic Games Store Ecom integration - Use IOnlineEntitlements, IOnlinePurchase and IOnlineStoreV2 for a complete feature set of the Ecom interface in the EOS SDK */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "EOS Ecom Settings", DisplayName = "Use new Ecom flow")
	bool bUseNewEcomFlow = false;

	/** Set to true to use new EOS login flow */
	UE_DEPRECATED(5.7, "bUseNewLoginFlow is deprecated, the legacy login flow has been removed from UserManagerEOS.")
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="EOS Login Settings", DisplayName="Use new Login flow, which doesn't rely on EOSPlus")
	bool bUseNewLoginFlow = false;

	/**
	 * When running with Steam, defines what TokenType OSSEOS will request from OSSSteam to login with.
	 * Please see EOS documentation at https://dev.epicgames.com/docs/dev-portal/identity-provider-management#steam for more information.
	 * Note the default is currently "Session" but this is deprecated. Please migrate to WebApi.
	 * Possible values:
	 *     "App" -> [DEPRECATED] Use Steam Encryption Application Tickets from ISteamUser::GetEncryptedAppTicket.
	 *     "Session" -> [DEPRECATED] Use Steam Auth Session Tickets from ISteamUser::GetAuthSessionTicket.
	 *     "WebApi" -> Use Steam Auth Tickets from ISteamUser::GetAuthTicketForWebApi, using the default remote service identity configured for OSSSteam.
	 *     "WebApi:<remoteserviceidentity>" -> Use Steam Auth Tickets from ISteamUser::GetAuthTicketForWebApi, using an explicit remote service identity.
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="Crossplay Settings")
	FString SteamTokenType = TEXT("Session");

	/**
	 * When running with Nintendo, defines what ExternalType will be used during ExternalAuth Login.
	 * The default is currently "NintendoServiceAccount".
	 * Possible values:
	 *     "NintendoServiceAccount" -> Use the EOS_ECT_NINTENDO_NSA_ID_TOKEN token type.
	 *     "NintendoAccount" -> Use the EOS_ECT_NINTENDO_ID_TOKEN token type.
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category="Crossplay Settings")
	FString NintendoTokenType = TEXT("NintendoServiceAccount");

	/** Get the settings for the selected artifact */
	static UE_API bool GetSelectedArtifactSettings(FEOSArtifactSettings& OutSettings);

	static UE_API FEOSSettings GetSettings();
	UE_API FEOSSettings ToNative() const;

private:
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	static UE_API FString GetDefaultArtifactName();

	static UE_API bool GetArtifactSettings(const FString& ArtifactName, FEOSArtifactSettings& OutSettings);
	static UE_API bool GetArtifactSettings(const FString& ArtifactName, const FString& SandboxId, FEOSArtifactSettings& OutSettings);
	static UE_API bool GetArtifactSettings(const FString& ArtifactName, const FString& SandboxId, const FString& DeploymentId, FEOSArtifactSettings& OutSettings);
	static UE_API bool GetArtifactSettingsImpl(const FString& ArtifactName, const TOptional<FString>& SandboxId, const TOptional<FString>& DeploymentId, FEOSArtifactSettings& OutSettings);

	static UE_API const TArray<FEOSArtifactSettings>& GetCachedArtifactSettings();

	static UE_API FEOSSettings AutoGetSettings();
	static UE_API const FEOSSettings& ManualGetSettings();

	friend class FOnlineSubsystemEOSModule;
	static UE_API void ModuleInit();
	static UE_API void ModuleShutdown();
};

#undef UE_API
