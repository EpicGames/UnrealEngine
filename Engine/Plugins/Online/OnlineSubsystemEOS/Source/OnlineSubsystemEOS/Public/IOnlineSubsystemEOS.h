// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemImpl.h"
#include "OnlineDelegateMacros.h"

class FUniqueNetId;
class IVoiceChatUser;
using IEOSPlatformHandlePtr = TSharedPtr<class IEOSPlatformHandle, ESPMode::ThreadSafe>;
using IOnlinePlayerSanctionEOSPtr = TSharedPtr<class IOnlinePlayerSanctionEOS, ESPMode::ThreadSafe>;
using IOnlinePlayerReportEOSPtr = TSharedPtr<class IOnlinePlayerReportEOS, ESPMode::ThreadSafe>;

using FUniqueNetIdEOSRef = TSharedRef<const class FUniqueNetIdEOS>;
using EOS_ProductUserId = struct EOS_ProductUserIdDetails*;

DECLARE_DELEGATE_TwoParams(FOnQueryUniqueNetIdComplete, FUniqueNetIdEOSRef /*ResolvedUniqueNetId*/, const FOnlineError& /*Error*/);

/**
 *	OnlineSubsystemEOS - Implementation of the online subsystem for EOS services
 */
class IOnlineSubsystemEOS : 
	public FOnlineSubsystemImpl
{
public:
	IOnlineSubsystemEOS(FName InSubsystemName, FName InInstanceName) : FOnlineSubsystemImpl(InSubsystemName, InInstanceName) {}
	virtual ~IOnlineSubsystemEOS() = default;

	virtual IVoiceChatUser* GetVoiceChatUserInterface(const FUniqueNetId& LocalUserId) = 0;
	virtual IEOSPlatformHandlePtr GetEOSPlatformHandle() const = 0;
	virtual IOnlinePlayerSanctionEOSPtr GetPlayerSanctionEOSInterface() const = 0;
	virtual IOnlinePlayerReportEOSPtr GetPlayerReportEOSInterface() const = 0;

	virtual void QueryUniqueNetId(int32 LocalUserNum, const EOS_ProductUserId& ProductUserId, const FOnQueryUniqueNetIdComplete & Callback) = 0;
};
