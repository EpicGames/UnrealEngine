// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "OnlineSubsystemTypes.h"
#include "OnlineDelegateMacros.h"

/**
 * Delegate fired when the player report report was sent
 *
 *  @param bWasSuccessful true if the player report was successfully received
 */
DECLARE_DELEGATE_OneParam(FOnSendPlayerReportComplete, const bool /* bWasSuccessful */);

/**
 * Public Interface for interacting with EOS player reports
 */
class IOnlinePlayerReportEOS
{
public:
	virtual ~IOnlinePlayerReportEOS() = default;

	/**
	*	Enum to set the category on the player report
	*/
	enum class EPlayerReportCategory
	{
		Cheating,
		Exploiting,
		OffensiveProfile,
		VerbalAbuse,
		Scamming,
		Spamming,
		Other
	};

	/**
	*	Struct for player report data
	*/
	struct FSendPlayerReportSettings
	{
		/** Required - The category of the player */
		EPlayerReportCategory Category;

		/** Optional - Message from the player descripting the issue being reported */
		FString Message;

		/** Optional - Context information around the issue being reported the game can send. This needs to be in a valid JSON format otherwise the EOS SDK will throw an error */
		FString Context;
	};

	/**
	* Send a player report for a player cheating, misconduct, etc... 
	* @param LocalUserId the player id of the player sending the player report
	* @param TargetUserId the player id the report is for
	* @param FSendPlayerReportSettings the data to include in the player report.
	*/
	virtual void SendPlayerReport(const FUniqueNetId& LocalUserId, const FUniqueNetId& TargetUserId, FSendPlayerReportSettings&& SendPlayerReportSettings, FOnSendPlayerReportComplete&& Delegate) = 0;
};
typedef TSharedPtr<IOnlinePlayerReportEOS, ESPMode::ThreadSafe> IOnlinePlayerReportEOSPtr;
