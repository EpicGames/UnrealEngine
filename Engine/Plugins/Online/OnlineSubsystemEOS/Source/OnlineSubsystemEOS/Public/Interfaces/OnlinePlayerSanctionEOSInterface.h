// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "OnlineSubsystemTypes.h"

/**
 * Delegate fired when a player sanction appeal has been created
 *
 *  @param bWasSuccessful true if creating the sanction appel was successful
 */
DECLARE_DELEGATE_OneParam(FOnCreatePlayerSanctionAppealComplete, const bool /* bWasSuccessful */);

/**
 * Delegate fired when player sanctions have been cached locally
 *
 *  @param bWasSuccessful true if sanctions are available in local cache
 */
DECLARE_DELEGATE_OneParam(FOnQueryActivePlayerSanctionsComplete, const bool /* bWasSuccessful */);

/**
 * Public Interface for interacting with EOS player sanctions
 */
class IOnlinePlayerSanctionEOS
{
public:
	virtual ~IOnlinePlayerSanctionEOS() = default;


	/**
	*	Enum to set the appeal reason when creating a sanction appeal
	*/
	enum class EPlayerSanctionAppealReason
	{
		IncorrectSanction,
		CompromisedAccount,
		UnfairPunishment,
		AppealForForgivenesss
	};

	/**
	*	Struct for the settings of sanction appeal
	*/
	struct FPlayerSanctionAppealSettings
	{
		/** The reason of the appeal */
		EPlayerSanctionAppealReason Reason;

		/** The sanction id for the sanction that is being appealed */
		FString ReferenceId;
	};

	/**
	*	Struct for player sanctions
	*/
	struct FOnlinePlayerSanction
	{
		/** The time the sanction was placed */
		int64_t TimePlaced;

		/** The time the sanction expires */
		int64_t TimeExpires;

		/** The action associated with this sanction */
		FString Action;

		/** The sanction id for the sanction that is being appealed. This needs to be set */
		FString ReferenceId;
	};

	/**
	* Send a player sanction appeal
	* @param LocalUserId the player id of the local user sending the sanction appeal. 
	* @param SanctionAppealSettings the settings needed to create the sanction appeal.
	 */
	virtual void CreatePlayerSanctionAppeal(const FUniqueNetId& LocalUserId, FPlayerSanctionAppealSettings&& SanctionAppealSettings, FOnCreatePlayerSanctionAppealComplete&& Delegate) = 0;

	/**
	* Query active player sanction. The sanctions will be cached locally and can be retrieved using GetCachedActivePlayerSanctions
	* @param LocalUserId the player id of the local user querying the sanctions. 
	* @param TargetUserId the player id of the player we are querying sanctions for. 
	 */
	virtual void QueryActivePlayerSanctions(const FUniqueNetId& LocalUserId, const FUniqueNetId& TargetUserId, FOnQueryActivePlayerSanctionsComplete&& Delegate) = 0;

	/**
	* Retrieve cached player sanctions
	* @param TargetUserId the player id we want to retrieve the sanctions for.
	* @param OutPlayerSanctions the array to store the sanctions.
	*/
	virtual EOnlineCachedResult::Type GetCachedActivePlayerSanctions(const FUniqueNetId& TargetUserId, TArray<FOnlinePlayerSanction>& OutPlayerSanctions) = 0;
};
typedef TSharedPtr<IOnlinePlayerSanctionEOS, ESPMode::ThreadSafe> IOnlinePlayerSanctionEOSPtr;
