// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EOS_SDK

#include "Online/CoreOnline.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_base.h"
#include "eos_common.h"

/** Unique net id for OnlineSubsystemEOS */
class IUniqueNetIdEOS : public FUniqueNetId
{
public:
	/**
	 * Get the Epic Account Id. May be null.
	 * @return the Epic Account Id, or null.
	 */
	virtual const EOS_EpicAccountId GetEpicAccountId() const = 0;
	/**
	 * Get the Product User Id. May be null.
	 * @return the Product User Id, or null.
	 */
	virtual const EOS_ProductUserId GetProductUserId() const = 0;
};

#define OSSEOS_BUCKET_ID_ATTRIBUTE_KEY FName("OSSEOS_BUCKET_ID_ATTRIBUTE_KEY")

#endif // WITH_EOS_SDK